// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldStreamingInsightsAnalyzer.h"
#include "HAL/LowLevelMemTracker.h"
#include "Model/WorldStreamingInsightsProvider.h"
#include "WorldStreamingInsightsLog.h"

FWorldStreamingInsightsAnalyzer::FWorldStreamingInsightsAnalyzer(TraceServices::IAnalysisSession& InSession, FWorldStreamingInsightsProvider& InProvider)
	: Session(InSession)
	, Provider(InProvider)
{
}

void FWorldStreamingInsightsAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	FInterfaceBuilder& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_WorldInitialization, "WorldStreaming", "WorldInitialization");
	Builder.RouteEvent(RouteId_WorldDeinitialization, "WorldStreaming", "WorldDeinitialization");
	Builder.RouteEvent(RouteId_ContainerDescription, "WorldStreaming", "ContainerDescription");
	Builder.RouteEvent(RouteId_ContainerStateChange, "WorldStreaming", "ContainerStateChange");
	Builder.RouteEvent(RouteId_StreamingSourceDescription, "WorldStreaming", "StreamingSourceDescription");
	Builder.RouteEvent(RouteId_StreamingSourceUpdate, "WorldStreaming", "StreamingSourceUpdate");
	Builder.RouteEvent(RouteId_StreamingSourceDeactivation, "WorldStreaming", "StreamingSourceDeactivation");
	Builder.RouteEvent(RouteId_TagGroupDescription, "WorldStreaming", "TagGroupDescription");
	Builder.RouteEvent(RouteId_TagDescription, "WorldStreaming", "TagDescription");
	Builder.RouteEvent(RouteId_ContainerPriorityUpdate, "WorldStreaming", "ContainerPriorityUpdate");
	Builder.RouteEvent(RouteId_PackageNameMapping, "WorldStreaming", "PackageNameMapping");
	Builder.RouteEvent(RouteId_ContainerDependencies, "WorldStreaming", "ContainerDependencies");
}

bool FWorldStreamingInsightsAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FWorldStreamingInsightsAnalyzer"));

	TraceServices::FAnalysisSessionEditScope Scope(Session);

	const FEventData& EventData = Context.EventData;

	switch (RouteId)
	{
	case RouteId_WorldInitialization:
	{
		uint64 WorldId = EventData.GetValue<uint64>("WorldId");
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		FWideStringView MapName;
		EventData.GetString("MapName", MapName);
		uint8 NetMode = EventData.GetValue<uint8>("NetMode", 0);

		Provider.AppendStreamingWorldStart(WorldId, Session.StoreString(MapName), static_cast<EStreamingWorldNetMode>(NetMode), Context.EventTime.AsSeconds(Cycle));
		break;
	}

	case RouteId_WorldDeinitialization:
	{
		uint64 WorldId = EventData.GetValue<uint64>("WorldId");
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");

		Provider.AppendStreamingWorldEnd(WorldId, Context.EventTime.AsSeconds(Cycle));
		break;
	}

	case RouteId_ContainerDescription:
	{
		uint64 Id = EventData.GetValue<uint64>("Id");
		uint64 WorldId = EventData.GetValue<uint64>("WorldId");
		uint64 ParentId = EventData.GetValue<uint64>("ParentId");

		FWideStringView Name;
		EventData.GetString("Name", Name);

		FWideStringView PackageName;
		EventData.GetString("PackageName", PackageName);

		TArrayView<const double> BoundsArray = EventData.GetArrayView<double>("Bounds");
		if (BoundsArray.Num() != 6)
		{
			UE_LOGF(LogWorldStreamingInsights, Warning, "Malformed ContainerDescription event: expected 6 bounds values, got %d. Skipping.", BoundsArray.Num());
			break;
		}

		TOptional<FBox> Bounds;
		if (EventData.GetValue<uint8>("bBoundsValid", 0) != 0)
		{
			Bounds = FBox(FVector(BoundsArray[0], BoundsArray[1], BoundsArray[2]), FVector(BoundsArray[3], BoundsArray[4], BoundsArray[5]));
		}

		TArrayView<const uint64> TagsArray = EventData.GetArrayView<uint64>("Tags");

		Provider.AppendStreamingContainerDescription(Id, WorldId, ParentId, Session.StoreString(Name), Session.StoreString(PackageName), MoveTemp(Bounds), TagsArray);
		break;
	}

	case RouteId_ContainerStateChange:
	{
		uint64 Id = EventData.GetValue<uint64>("Id");
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 WorldId = EventData.GetValue<uint64>("WorldId");
		uint8 NewState = EventData.GetValue<uint8>("NewState");

		Provider.AppendStreamingContainerStateChange(WorldId, Context.EventTime.AsSeconds(Cycle), Id, static_cast<EStreamingContainerState>(NewState));
		break;
	}

	case RouteId_StreamingSourceDescription:
	{
		uint64 Id = EventData.GetValue<uint64>("Id");
		uint64 WorldId = EventData.GetValue<uint64>("WorldId");

		FWideStringView Name;
		EventData.GetString("Name", Name);

		if (Name.IsEmpty())
		{
			UE_LOGF(LogWorldStreamingInsights, Warning, "Streaming source with Id 0x%llX has no name. Multiple unnamed sources will collide in the trace. Assign a unique name to each streaming source provider.", Id);
		}

		Provider.AppendStreamingSourceDescription(Id, WorldId, Session.StoreString(Name));
		break;
	}

	case RouteId_StreamingSourceUpdate:
	{
		uint64 Id = EventData.GetValue<uint64>("Id");
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 WorldId = EventData.GetValue<uint64>("WorldId");

		TArrayView<const double> LocationArray = EventData.GetArrayView<double>("Location");
		if (LocationArray.Num() != 3)
		{
			UE_LOGF(LogWorldStreamingInsights, Warning, "Malformed StreamingSourceUpdate event: expected 3 location values, got %d. Skipping.", LocationArray.Num());
			break;
		}
		FVector Location(LocationArray[0], LocationArray[1], LocationArray[2]);

		Provider.AppendStreamingSourceUpdate(WorldId, Context.EventTime.AsSeconds(Cycle), Id, Location);
		break;
	}

	case RouteId_StreamingSourceDeactivation:
	{
		uint64 Id = EventData.GetValue<uint64>("Id");
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 WorldId = EventData.GetValue<uint64>("WorldId");

		Provider.AppendStreamingSourceDeactivation(WorldId, Context.EventTime.AsSeconds(Cycle), Id);

		break;
	}

	case RouteId_TagGroupDescription:
	{
		uint64 Id = EventData.GetValue<uint64>("Id");

		FWideStringView Name;
		EventData.GetString("Name", Name);

		Provider.AppendTagGroup(Id, Session.StoreString(Name));
		break;
	}

	case RouteId_TagDescription:
	{
		uint64 Id = EventData.GetValue<uint64>("Id");
		uint64 GroupId = EventData.GetValue<uint64>("GroupId");
		uint64 ParentId = EventData.GetValue<uint64>("ParentId");

		FWideStringView Name;
		EventData.GetString("Name", Name);

		Provider.AppendTag(Id, GroupId, ParentId, Session.StoreString(Name));
		break;
	}

	case RouteId_ContainerPriorityUpdate:
	{
		uint64 WorldId = EventData.GetValue<uint64>("WorldId");
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");

		TArrayView<const uint64> ContainerIds = EventData.GetArrayView<uint64>("ContainerIds");
		TArrayView<const float> Priorities = EventData.GetArrayView<float>("Priorities");

		if (ContainerIds.Num() != Priorities.Num())
		{
			UE_LOGF(LogWorldStreamingInsights, Warning, "Malformed ContainerPriorityUpdate event: ContainerIds count (%d) != Priorities count (%d). Skipping.", ContainerIds.Num(), Priorities.Num());
			break;
		}

		Provider.AppendStreamingContainerPriorities(WorldId, Context.EventTime.AsSeconds(Cycle), ContainerIds, Priorities);
		break;
	}

	case RouteId_PackageNameMapping:
	{
		uint64 PackageId = EventData.GetValue<uint64>("PackageId");
		FWideStringView Name;
		EventData.GetString("Name", Name);
		Provider.AppendPackageNameMapping(PackageId, Session.StoreString(Name));
		break;
	}

	case RouteId_ContainerDependencies:
	{
		uint64 Id = EventData.GetValue<uint64>("Id");
		uint64 WorldId = EventData.GetValue<uint64>("WorldId");
		TArrayView<const uint64> DependencyIds = EventData.GetArrayView<uint64>("DependencyIds");
		Provider.AppendContainerDependencies(Id, WorldId, DependencyIds);
		break;
	}
	}
	return true;
}