// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"

namespace TraceServices
{
	class IAnalysisSession;
}

class FWorldStreamingInsightsProvider;

class FWorldStreamingInsightsAnalyzer : public UE::Trace::IAnalyzer
{
public:
	FWorldStreamingInsightsAnalyzer(TraceServices::IAnalysisSession& InSession, FWorldStreamingInsightsProvider& InProvider);

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_WorldInitialization,
		RouteId_WorldDeinitialization,
		RouteId_ContainerDescription,
		RouteId_ContainerStateChange,
		RouteId_StreamingSourceDescription,
		RouteId_StreamingSourceUpdate,
		RouteId_StreamingSourceDeactivation,
		RouteId_TagGroupDescription,
		RouteId_TagDescription,
		RouteId_ContainerPriorityUpdate,
		RouteId_PackageNameMapping,
		RouteId_ContainerDependencies
	};

	TraceServices::IAnalysisSession& Session;
	FWorldStreamingInsightsProvider& Provider;
};