// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldStreamingInsightsProvider.h"

#include "TraceServices/Model/AnalysisSession.h"
#include "WorldStreamingInsightsLog.h"

FWorldStreamingInsightsProvider::FWorldStreamingInsightsProvider(TraceServices::IAnalysisSession& InSession)
	: Session(InSession)
{
}

void FWorldStreamingInsightsProvider::AppendStreamingWorldStart(uint64 InWorldId, const TCHAR* InName, EStreamingWorldNetMode InNetMode, double InTimestamp)
{
	Session.WriteAccessCheck();

	if (Worlds.Contains(InWorldId))
	{
		UE_LOGF(LogWorldStreamingInsights, Warning, "Duplicate WorldId 0x%llX (%ls). Previous world data may be corrupt.", InWorldId, InName);
	}

	FStreamingWorldInfo& StreamingWorldInfo = Worlds.Add(InWorldId);
	StreamingWorldInfo.WorldId = InWorldId;
	StreamingWorldInfo.MapName = InName;
	StreamingWorldInfo.NetMode = InNetMode;
	StreamingWorldInfo.StartTime = InTimestamp;
	StreamingWorldInfo.EndTime = DBL_MAX;

	ChangeSerial++;
}

void FWorldStreamingInsightsProvider::AppendStreamingWorldEnd(uint64 InWorldId, double InTimestamp)
{
	Session.WriteAccessCheck();

	if (FStreamingWorldInfo* StreamingWorldInfoPtr = Worlds.Find(InWorldId))
	{
		StreamingWorldInfoPtr->EndTime = InTimestamp;
	}
	else
	{
		UE_LOGF(LogWorldStreamingInsights, Warning, "WorldEnd event for unknown WorldId 0x%llX. WorldStart event may have been lost.", InWorldId);
	}

	if (TMap<uint64, int32>* ActiveStreamingSourceLookupPtr = ActiveStreamingSourceLookupPerWorld.Find(InWorldId))
	{
		if (TArray<FStreamingSourceInfo>* StreamingSourcesPtr = StreamingSourcesPerWorld.Find(InWorldId))
		{
			TArray<FStreamingSourceInfo>& StreamingSources = *StreamingSourcesPtr;
			for (const TPair<uint64, int32>& Entry : *ActiveStreamingSourceLookupPtr)
			{
				int32 StreamingSourceIndex = Entry.Value;
				ensure(StreamingSources[StreamingSourceIndex].EndTime == DBL_MAX);
				StreamingSources[StreamingSourceIndex].EndTime = InTimestamp;
			}
		}
		ActiveStreamingSourceLookupPtr->Reset();
	}

	ChangeSerial++;
}

void FWorldStreamingInsightsProvider::AppendStreamingContainerDescription(uint64 InContainerId, uint64 InWorldId, uint64 InParentId, const TCHAR* InName, const TCHAR* InPackageName, TOptional<FBox> InBounds, TArrayView<const uint64> InTags)
{
	Session.WriteAccessCheck();

	TMap<uint64, FStreamingContainerInfo>& ContainersMap = ContainersPerWorld.FindOrAdd(InWorldId);
	FStreamingContainerInfo& StreamingContainerInfo = ContainersMap.Add(InContainerId);

	StreamingContainerInfo.ContainerId = InContainerId;
	StreamingContainerInfo.ParentId = InParentId;
	StreamingContainerInfo.Name = InName;
	StreamingContainerInfo.PackageName = InPackageName;
	StreamingContainerInfo.Bounds = MoveTemp(InBounds);
	StreamingContainerInfo.Tags = InTags;

	if (InParentId != 0)
	{
		TMap<uint64, TSet<uint64> >& ContainerToChildContainers = ContainerToChildContainersPerWorld.FindOrAdd(InWorldId);
		TSet<uint64>& ChildContainers = ContainerToChildContainers.FindOrAdd(InParentId);
		ChildContainers.Add(InContainerId);
	}
	else
	{
		TSet<uint64>& RootContainers = RootContainersPerWorld.FindOrAdd(InWorldId);
		RootContainers.Add(InContainerId);
	}

	ChangeSerial++;
}

void FWorldStreamingInsightsProvider::AppendStreamingSourceDescription(uint64 InStreamingSourceId, uint64 InWorldId, const TCHAR* InName)
{
	Session.WriteAccessCheck();

	TArray<FStreamingSourceInfo>& StreamingSources = StreamingSourcesPerWorld.FindOrAdd(InWorldId);
	FStreamingSourceInfo& StreamingSourceInfo = StreamingSources.AddDefaulted_GetRef();

	StreamingSourceInfo.SourceId = InStreamingSourceId;
	StreamingSourceInfo.Name = InName;
	StreamingSourceInfo.StartTime = DBL_MAX;
	StreamingSourceInfo.EndTime = DBL_MAX;

	TMap<uint64, int32>& ActiveStreamingSourcesLookup = ActiveStreamingSourceLookupPerWorld.FindOrAdd(InWorldId);
	if (ActiveStreamingSourcesLookup.Contains(InStreamingSourceId))
	{
		UE_LOGF(LogWorldStreamingInsights, Warning, "Duplicate StreamingSourceId 0x%llX for WorldId 0x%llX. Previous source data may be corrupt.", InStreamingSourceId, InWorldId);
	}
	ActiveStreamingSourcesLookup.Add(InStreamingSourceId, StreamingSources.Num() - 1);

	SourceNamesPerWorld.FindOrAdd(InWorldId).Add(InStreamingSourceId, InName);

	ChangeSerial++;
}

void FWorldStreamingInsightsProvider::AppendTagGroup(uint64 InGroupId, const TCHAR* InName)
{
	Session.WriteAccessCheck();

	FStreamingTagGroup& TagGroup = TagGroups.Add(InGroupId);

	TagGroup.GroupId = InGroupId;
	TagGroup.Name = InName;
	TagGroup.UntaggedLabel = Session.StoreString(FString::Printf(TEXT("<no %s>"), InName));

	ChangeSerial++;
}

void FWorldStreamingInsightsProvider::AppendTag(uint64 InTagId, uint64 InTagGroupId, uint64 InParentId, const TCHAR* InName)
{
	Session.WriteAccessCheck();

	FStreamingTag& Tag = Tags.Add(InTagId);

	Tag.TagId = InTagId;
	Tag.GroupId = InTagGroupId;
	Tag.ParentId = InParentId;
	Tag.Name = InName;

	TagsPerGroup.FindOrAdd(InTagGroupId).Add(InTagId);

	ChangeSerial++;
}

void FWorldStreamingInsightsProvider::AppendStreamingContainerStateChange(uint64 InWorldId, double InTimestamp, uint64 InContainerId, EStreamingContainerState InNewState)
{
	Session.WriteAccessCheck();

	TArray<FStreamingContainerStateChange>& ContainerStateChanges = ContainerStateChangesPerWorld.FindOrAdd(InWorldId);
	FStreamingContainerStateChange& StateChange = ContainerStateChanges.AddDefaulted_GetRef();

	StateChange.ContainerId = InContainerId;
	StateChange.Timestamp = InTimestamp;
	StateChange.NewState = InNewState;

	TMap<uint64, TArray<int32> >& ContainerToStateChanges = ContainerToStateChangesPerWorld.FindOrAdd(InWorldId);
	TArray<int32>& StateChanges = ContainerToStateChanges.FindOrAdd(InContainerId);
	StateChanges.Add(ContainerStateChanges.Num() - 1);

	ChangeSerial++;
}

void FWorldStreamingInsightsProvider::AppendStreamingContainerPriorities(uint64 InWorldId, double InTimestamp, TArrayView<const uint64> InContainerIds, TArrayView<const float> InPriorities)
{
	Session.WriteAccessCheck();

	ensure(InContainerIds.Num() == InPriorities.Num());
	if (InContainerIds.Num() == 0)
	{
		return;
	}

	TMap<uint64, TArray<FStreamingContainerPriorityUpdate> >& ContainerPriorities = ContainerPrioritiesPerWorld.FindOrAdd(InWorldId);
	for (int32 Index = 0; Index < InContainerIds.Num(); ++Index)
	{
		TArray<FStreamingContainerPriorityUpdate>& Updates = ContainerPriorities.FindOrAdd(InContainerIds[Index]);
		FStreamingContainerPriorityUpdate& Update = Updates.AddDefaulted_GetRef();
		Update.Timestamp = InTimestamp;
		Update.Priority = InPriorities[Index];
	}

	ChangeSerial++;
}

void FWorldStreamingInsightsProvider::AppendStreamingSourceUpdate(uint64 InWorldId, double InTimestamp, uint64 InSourceId, FVector InLocation)
{
	Session.WriteAccessCheck();

	TArray<FStreamingSourceUpdate>& StreamingSourceUpdates = StreamingSourceUpdatesPerWorld.FindOrAdd(InWorldId);
	FStreamingSourceUpdate& StreamingSourceUpdate = StreamingSourceUpdates.AddDefaulted_GetRef();

	StreamingSourceUpdate.SourceId = InSourceId;
	StreamingSourceUpdate.Timestamp = InTimestamp;
	StreamingSourceUpdate.Location = InLocation;

	TMap<uint64, TArray<int32> >& StreamingSourceToUpdates = StreamingSourceToUpdatesPerWorld.FindOrAdd(InWorldId);
	TArray<int32>& Updates = StreamingSourceToUpdates.FindOrAdd(InSourceId);
	Updates.Add(StreamingSourceUpdates.Num() - 1);

	TMap<uint64, int32>* ActiveStreamingSourceLookupPtr = ActiveStreamingSourceLookupPerWorld.Find(InWorldId);
	int32* StreamingSourceIndexPtr = ActiveStreamingSourceLookupPtr ? ActiveStreamingSourceLookupPtr->Find(InSourceId) : nullptr;

	if (StreamingSourceIndexPtr)
	{
		int32 StreamingSourceIndex = *StreamingSourceIndexPtr;
		if (TArray<FStreamingSourceInfo>* StreamingSourcesPtr = StreamingSourcesPerWorld.Find(InWorldId))
		{
			TArray<FStreamingSourceInfo>& StreamingSources = *StreamingSourcesPtr;
			if (StreamingSources[StreamingSourceIndex].StartTime == DBL_MAX)
			{
				StreamingSources[StreamingSourceIndex].StartTime = InTimestamp;
			}
		}
	}
	else if (TArray<FStreamingSourceInfo>* StreamingSourcesPtr = StreamingSourcesPerWorld.Find(InWorldId))
	{
		// Source was previously deactivated; create a new activation period.
		const TMap<uint64, const TCHAR*>* SourceNames = SourceNamesPerWorld.Find(InWorldId);
		const TCHAR* SourceName = SourceNames ? SourceNames->FindRef(InSourceId) : nullptr;

		if (SourceName)
		{
			TArray<FStreamingSourceInfo>& StreamingSources = *StreamingSourcesPtr;
			FStreamingSourceInfo& NewInfo = StreamingSources.AddDefaulted_GetRef();
			NewInfo.SourceId = InSourceId;
			NewInfo.Name = SourceName;
			NewInfo.StartTime = InTimestamp;
			NewInfo.EndTime = DBL_MAX;

			TMap<uint64, int32>& ActiveLookup = ActiveStreamingSourceLookupPerWorld.FindOrAdd(InWorldId);
			ActiveLookup.Add(InSourceId, StreamingSources.Num() - 1);
		}
		else
		{
			UE_LOGF(LogWorldStreamingInsights, Warning, "StreamingSourceUpdate for unknown SourceId 0x%llX in WorldId 0x%llX. StreamingSourceDesc event may have been lost.", InSourceId, InWorldId);
		}
	}

	ChangeSerial++;
}

void FWorldStreamingInsightsProvider::AppendStreamingSourceDeactivation(uint64 InWorldId, double InTimestamp, uint64 InSourceId)
{
	Session.WriteAccessCheck();

	if (TMap<uint64, int32>* ActiveStreamingSourceLookupPtr = ActiveStreamingSourceLookupPerWorld.Find(InWorldId))
	{
		if (int32* StreamingSourceIndexPtr = ActiveStreamingSourceLookupPtr->Find(InSourceId))
		{
			int32 StreamingSourceIndex = *StreamingSourceIndexPtr;
			if (TArray<FStreamingSourceInfo>* StreamingSourcesPtr = StreamingSourcesPerWorld.Find(InWorldId))
			{
				TArray<FStreamingSourceInfo>& StreamingSources = *StreamingSourcesPtr;
				ensure(StreamingSources[StreamingSourceIndex].EndTime == DBL_MAX);
				StreamingSources[StreamingSourceIndex].EndTime = InTimestamp;
			}
		}
		ActiveStreamingSourceLookupPtr->Remove(InSourceId);
	}

	ChangeSerial++;
}

void FWorldStreamingInsightsProvider::AppendPackageNameMapping(uint64 InPackageId, const TCHAR* InName)
{
	Session.WriteAccessCheck();

	PackageNames.Add(InPackageId, InName);
	ChangeSerial++;
}

void FWorldStreamingInsightsProvider::AppendContainerDependencies(uint64 InContainerId, uint64 InWorldId, TArrayView<const uint64> InDependencyIds)
{
	Session.WriteAccessCheck();

	if (TMap<uint64, FStreamingContainerInfo>* Containers = ContainersPerWorld.Find(InWorldId))
	{
		if (FStreamingContainerInfo* ContainerInfo = Containers->Find(InContainerId))
		{
			ContainerInfo->DependencyPackageIds.Append(InDependencyIds.GetData(), InDependencyIds.Num());
			ChangeSerial++;
		}
		else
		{
			UE_LOGF(LogWorldStreamingInsights, Warning, "ContainerDependencies for unknown ContainerId 0x%llX in WorldId 0x%llX. ContainerDescription event may have been lost.", InContainerId, InWorldId);
		}
	}
}

const TCHAR* FWorldStreamingInsightsProvider::GetPackageName(uint64 InPackageId) const
{
	Session.ReadAccessCheck();

	return PackageNames.FindRef(InPackageId);
}

uint32 FWorldStreamingInsightsProvider::GetChangeSerial() const
{
	Session.ReadAccessCheck();

	return ChangeSerial;
}

uint32 FWorldStreamingInsightsProvider::GetStreamingWorldCount() const
{
	Session.ReadAccessCheck();

	return Worlds.Num();
}

void FWorldStreamingInsightsProvider::EnumerateStreamingWorlds(TFunctionRef<void(const FStreamingWorldInfo&)> InCallback) const
{
	Session.ReadAccessCheck();

	for (const TPair<uint64, FStreamingWorldInfo>& Entry : Worlds)
	{
		InCallback(Entry.Value);
	}
}

void FWorldStreamingInsightsProvider::EnumerateStreamingWorldsAtTime(double InTime, TFunctionRef<void(const FStreamingWorldInfo&)> InCallback) const
{
	Session.ReadAccessCheck();

	for (const TPair<uint64, FStreamingWorldInfo>& Entry : Worlds)
	{
		const FStreamingWorldInfo& WorldInfo = Entry.Value;
		if (InTime >= WorldInfo.StartTime && InTime <= WorldInfo.EndTime)
		{
			InCallback(WorldInfo);
		}
	}
}

bool FWorldStreamingInsightsProvider::ReadStreamingWorld(uint64 InWorldId, TFunctionRef<void(const FStreamingWorldInfo&)> InCallback) const
{
	Session.ReadAccessCheck();

	if (const FStreamingWorldInfo* StreamingWorldInfoPtr = Worlds.Find(InWorldId))
	{
		InCallback(*StreamingWorldInfoPtr);
		return true;
	}
	return false;
}

uint32 FWorldStreamingInsightsProvider::GetStreamingContainerCount(uint64 InWorldId) const
{
	Session.ReadAccessCheck();

	if (const TMap<uint64, FStreamingContainerInfo>* ContainersPtr = ContainersPerWorld.Find(InWorldId))
	{
		return ContainersPtr->Num();
	}
	return 0;
}

void FWorldStreamingInsightsProvider::EnumerateStreamingContainers(uint64 InWorldId, TFunctionRef<void(const FStreamingContainerInfo&)> InCallback) const
{
	Session.ReadAccessCheck();

	if (const TMap<uint64, FStreamingContainerInfo>* ContainersPtr = ContainersPerWorld.Find(InWorldId))
	{
		for (const TPair<uint64, FStreamingContainerInfo>& Entry : *ContainersPtr)
		{
			InCallback(Entry.Value);
		}
	}
}

bool FWorldStreamingInsightsProvider::ReadStreamingContainer(uint64 InWorldId, uint64 InContainerId, TFunctionRef<void(const FStreamingContainerInfo&)> InCallback) const
{
	Session.ReadAccessCheck();

	if (const TMap<uint64, FStreamingContainerInfo>* ContainersPtr = ContainersPerWorld.Find(InWorldId))
	{
		if (const FStreamingContainerInfo* StreamingContainerInfoPtr = ContainersPtr->Find(InContainerId))
		{
			InCallback(*StreamingContainerInfoPtr);
			return true;
		}
	}
	return false;
}

void FWorldStreamingInsightsProvider::EnumerateChildStreamingContainers(uint64 InWorldId, uint64 InContainerId, TFunctionRef<void(const FStreamingContainerInfo&)> InCallback) const
{
	Session.ReadAccessCheck();

	if (const TMap<uint64, TSet<uint64> >* ContainerToChildContainersPtr = ContainerToChildContainersPerWorld.Find(InWorldId))
	{
		if (const TSet<uint64>* ChildContainersPtr = ContainerToChildContainersPtr->Find(InContainerId))
		{
			for (uint64 ChildContainer : *ChildContainersPtr)
			{
				ReadStreamingContainer(InWorldId, ChildContainer, InCallback);
			}
		}
	}
}

void FWorldStreamingInsightsProvider::EnumerateRootStreamingContainers(uint64 InWorldId, TFunctionRef<void(const FStreamingContainerInfo&)> InCallback) const
{
	Session.ReadAccessCheck();

	if (const TSet<uint64>* RootContainersPtr = RootContainersPerWorld.Find(InWorldId))
	{
		for (const uint64& RootContainer : *RootContainersPtr)
		{
			ReadStreamingContainer(InWorldId, RootContainer, InCallback);
		}
	}
}

void FWorldStreamingInsightsProvider::EnumerateStreamingContainerStateChanges(uint64 InWorldId, uint64 InContainerId, double InStartTime, double InEndTime, TFunctionRef<void(const FStreamingContainerStateChange&)> InCallback) const
{
	Session.ReadAccessCheck();

	if (const TArray<FStreamingContainerStateChange>* ContainerStateChangesPtr = ContainerStateChangesPerWorld.Find(InWorldId))
	{
		const TArray<FStreamingContainerStateChange>& ContainerStateChanges = *ContainerStateChangesPtr;
		if (const TMap<uint64, TArray<int32> >* ContainerToStateChangesPtr = ContainerToStateChangesPerWorld.Find(InWorldId))
		{
			if (const TArray<int32>* StateChangesPtr = ContainerToStateChangesPtr->Find(InContainerId))
			{
				const TArray<int32>& StateChanges = *StateChangesPtr;
				const int32 StartIndex = Algo::LowerBoundBy(StateChanges, InStartTime, [&ContainerStateChanges](const int32& Index) { return ContainerStateChanges[Index].Timestamp; });

				for (int32 Index = StartIndex; Index < StateChanges.Num() && ContainerStateChanges[StateChanges[Index]].Timestamp <= InEndTime; Index++)
				{
					InCallback(ContainerStateChanges[StateChanges[Index]]);
				}
			}
		}
	}
}

EStreamingContainerState FWorldStreamingInsightsProvider::GetStreamingContainerStateAtTime(uint64 InWorldId, uint64 InContainerId, double InTime) const
{
	Session.ReadAccessCheck();

	if (const TArray<FStreamingContainerStateChange>* ContainerStateChangesPtr = ContainerStateChangesPerWorld.Find(InWorldId))
	{
		const TArray<FStreamingContainerStateChange>& ContainerStateChanges = *ContainerStateChangesPtr;
		if (const TMap<uint64, TArray<int32> >* ContainerToStateChangesPtr = ContainerToStateChangesPerWorld.Find(InWorldId))
		{
			if (const TArray<int32>* StateChangesPtr = ContainerToStateChangesPtr->Find(InContainerId))
			{
				const TArray<int32>& StateChanges = *StateChangesPtr;
				// UpperBoundBy returns the first index with timestamp > InTime; the most recent state change at or before InTime is one position earlier.
				const int32 StateChangeIndex = Algo::UpperBoundBy(StateChanges, InTime, [&ContainerStateChanges](const int32& Index) { return ContainerStateChanges[Index].Timestamp; });

				if (StateChangeIndex > 0)
				{
					check(ContainerStateChanges[StateChanges[StateChangeIndex - 1]].Timestamp <= InTime);
					return ContainerStateChanges[StateChanges[StateChangeIndex - 1]].NewState;
				}
			}
		}
	}

	return EStreamingContainerState::Unloaded;
}

float FWorldStreamingInsightsProvider::GetStreamingContainerPriorityAtTime(uint64 InWorldId, uint64 InContainerId, double InTime) const
{
	Session.ReadAccessCheck();

	if (const TMap<uint64, TArray<FStreamingContainerPriorityUpdate> >* ContainerPrioritiesPtr = ContainerPrioritiesPerWorld.Find(InWorldId))
	{
		if (const TArray<FStreamingContainerPriorityUpdate>* UpdatesPtr = ContainerPrioritiesPtr->Find(InContainerId))
		{
			const TArray<FStreamingContainerPriorityUpdate>& Updates = *UpdatesPtr;
			const int32 UpdateIndex = Algo::UpperBoundBy(Updates, InTime, [](const FStreamingContainerPriorityUpdate& Update) { return Update.Timestamp; });

			if (UpdateIndex > 0)
			{
				check(Updates[UpdateIndex - 1].Timestamp <= InTime);
				return FMath::Clamp(Updates[UpdateIndex - 1].Priority, 0.0f, 1.0f);
			}
		}
	}

	return -1.0f;
}

void FWorldStreamingInsightsProvider::EnumerateStreamingSources(uint64 InWorldId, TFunctionRef<void(const FStreamingSourceInfo&)> InCallback) const
{
	Session.ReadAccessCheck();

	if (const TArray<FStreamingSourceInfo>* StreamingSourcesPtr = StreamingSourcesPerWorld.Find(InWorldId))
	{
		for (const FStreamingSourceInfo& StreamingSource : *StreamingSourcesPtr)
		{
			InCallback(StreamingSource);
		}
	}
}

void FWorldStreamingInsightsProvider::EnumerateStreamingSourcesAtTime(uint64 InWorldId, double InTime, TFunctionRef<void(const FStreamingSourceInfo&)> InCallback) const
{
	Session.ReadAccessCheck();

	if (const TArray<FStreamingSourceInfo>* StreamingSourcesPtr = StreamingSourcesPerWorld.Find(InWorldId))
	{
		for (const FStreamingSourceInfo& StreamingSource : *StreamingSourcesPtr)
		{
			if (InTime >= StreamingSource.StartTime && InTime <= StreamingSource.EndTime)
			{
				InCallback(StreamingSource);
			}
		}
	}
}

bool FWorldStreamingInsightsProvider::ReadStreamingSourceAtTime(uint64 InWorldId, uint64 InSourceId, double InTime, TFunctionRef<void(const FStreamingSourceInfo&)> InCallback) const
{
	Session.ReadAccessCheck();

	if (const TArray<FStreamingSourceInfo>* StreamingSourcesPtr = StreamingSourcesPerWorld.Find(InWorldId))
	{
		for (const FStreamingSourceInfo& StreamingSource : *StreamingSourcesPtr)
		{
			if (InSourceId == StreamingSource.SourceId && InTime >= StreamingSource.StartTime && InTime <= StreamingSource.EndTime)
			{
				InCallback(StreamingSource);
				return true;
			}
		}
	}
	return false;
}

void FWorldStreamingInsightsProvider::EnumerateStreamingSourceUpdates(uint64 InWorldId, uint64 InSourceId, double InStartTime, double InEndTime, TFunctionRef<void(const FStreamingSourceUpdate&)> InCallback) const
{
	Session.ReadAccessCheck();

	if (const TArray<FStreamingSourceUpdate>* AllStreamingSourceUpdatesPtr = StreamingSourceUpdatesPerWorld.Find(InWorldId))
	{
		const TArray<FStreamingSourceUpdate>& AllStreamingSourceUpdates = *AllStreamingSourceUpdatesPtr;
		if (const TMap<uint64, TArray<int32> >* StreamingSourceToUpdatesPtr = StreamingSourceToUpdatesPerWorld.Find(InWorldId))
		{
			if (const TArray<int32>* StreamingSourceUpdatesPtr = StreamingSourceToUpdatesPtr->Find(InSourceId))
			{
				const TArray<int32>& StreamingSourceUpdates = *StreamingSourceUpdatesPtr;
				const int32 StartIndex = Algo::LowerBoundBy(StreamingSourceUpdates, InStartTime, [&AllStreamingSourceUpdates](const int32& Index) { return AllStreamingSourceUpdates[Index].Timestamp; });

				for (int32 Index = StartIndex; Index < StreamingSourceUpdates.Num() && AllStreamingSourceUpdates[StreamingSourceUpdates[Index]].Timestamp <= InEndTime; Index++)
				{
					InCallback(AllStreamingSourceUpdates[StreamingSourceUpdates[Index]]);
				}
			}
		}
	}
}

FVector FWorldStreamingInsightsProvider::GetStreamingSourceLocationAtTime(uint64 InWorldId, uint64 InSourceId, double InTime) const
{
	Session.ReadAccessCheck();

	if (const TArray<FStreamingSourceUpdate>* AllStreamingSourceUpdatesPtr = StreamingSourceUpdatesPerWorld.Find(InWorldId))
	{
		const TArray<FStreamingSourceUpdate>& AllStreamingSourceUpdates = *AllStreamingSourceUpdatesPtr;
		if (const TMap<uint64, TArray<int32> >* StreamingSourceToUpdatesPtr = StreamingSourceToUpdatesPerWorld.Find(InWorldId))
		{
			if (const TArray<int32>* StreamingSourceUpdatesPtr = StreamingSourceToUpdatesPtr->Find(InSourceId))
			{
				const TArray<int32>& StreamingSourceUpdates = *StreamingSourceUpdatesPtr;
				const int32 SourceUpdateIndex = Algo::UpperBoundBy(StreamingSourceUpdates, InTime, [&AllStreamingSourceUpdates](const int32& Index) { return AllStreamingSourceUpdates[Index].Timestamp; });

				if (SourceUpdateIndex > 0)
				{
					check(AllStreamingSourceUpdates[StreamingSourceUpdates[SourceUpdateIndex - 1]].Timestamp <= InTime);
					return AllStreamingSourceUpdates[StreamingSourceUpdates[SourceUpdateIndex - 1]].Location;
				}
			}
		}
	}

	return FVector::ZeroVector;
}

void FWorldStreamingInsightsProvider::EnumerateStreamingTagGroups(TFunctionRef<void(const FStreamingTagGroup&)> InCallback) const
{
	Session.ReadAccessCheck();

	for (const TPair<uint64, FStreamingTagGroup>& Entry : TagGroups)
	{
		InCallback(Entry.Value);
	}
}

bool FWorldStreamingInsightsProvider::ReadStreamingTagGroup(uint64 InGroupId, TFunctionRef<void(const FStreamingTagGroup&)> InCallback) const
{
	Session.ReadAccessCheck();

	if (const FStreamingTagGroup* TagGroup = TagGroups.Find(InGroupId))
	{
		InCallback(*TagGroup);
		return true;
	}
	return false;
}

void FWorldStreamingInsightsProvider::EnumerateStreamingTagsInGroup(uint64 InGroupId, TFunctionRef<void(const FStreamingTag&)> InCallback) const
{
	Session.ReadAccessCheck();

	if (const TSet<uint64>* TagsSetPtr = TagsPerGroup.Find(InGroupId))
	{
		for (uint64 Tag : *TagsSetPtr)
		{
			if (const FStreamingTag* StreamingTagPtr = Tags.Find(Tag))
			{
				InCallback(*StreamingTagPtr);
			}
		}
	}
}

bool FWorldStreamingInsightsProvider::ReadStreamingTag(uint64 InTagId, TFunctionRef<void(const FStreamingTag&)> InCallback) const
{
	Session.ReadAccessCheck();

	if (const FStreamingTag* StreamingTag = Tags.Find(InTagId))
	{
		InCallback(*StreamingTag);
		return true;
	}
	return false;
}

FName GetWorldStreamingInsightsProviderName()
{
	static const FName Name("WorldStreamingInsightsProvider");
	return Name;
}

const IWorldStreamingInsightsProvider* ReadWorldStreamingInsightsProvider(const TraceServices::IAnalysisSession& InSession)
{
	return InSession.ReadProvider<IWorldStreamingInsightsProvider>(GetWorldStreamingInsightsProviderName());
}