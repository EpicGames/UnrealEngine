// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/Model/AnalysisSession.h"
#include "IWorldStreamingInsightsProvider.h"

class FWorldStreamingInsightsProvider : public IWorldStreamingInsightsProvider
{
public:
	FWorldStreamingInsightsProvider(TraceServices::IAnalysisSession& InSession);

	void AppendStreamingWorldStart(uint64 InWorldId, const TCHAR* InName, EStreamingWorldNetMode InNetMode, double InTimestamp);
	void AppendStreamingWorldEnd(uint64 InWorldId, double InTimestamp);
	void AppendStreamingContainerDescription(uint64 InContainerId, uint64 InWorldId, uint64 InParentId, const TCHAR* InName, const TCHAR* InPackageName, TOptional<FBox> InBounds, TArrayView<const uint64> InTags);
	void AppendStreamingSourceDescription(uint64 InStreamingSourceId, uint64 InWorldId, const TCHAR* InName);
	void AppendTagGroup(uint64 InGroupId, const TCHAR* InName);
	void AppendTag(uint64 InTagId, uint64 InTagGroupId, uint64 InParentId, const TCHAR* InName);
	void AppendStreamingContainerStateChange(uint64 InWorldId, double InTimestamp, uint64 InContainerId, EStreamingContainerState InNewState);
	void AppendStreamingContainerPriorities(uint64 InWorldId, double InTimestamp, TArrayView<const uint64> InContainerIds, TArrayView<const float> InPriorities);
	void AppendStreamingSourceUpdate(uint64 InWorldId, double InTimestamp, uint64 InSourceId, FVector InLocation);
	void AppendStreamingSourceDeactivation(uint64 InWorldId, double InTimestamp, uint64 InSourceId);
	void AppendPackageNameMapping(uint64 InPackageId, const TCHAR* InName);
	void AppendContainerDependencies(uint64 InContainerId, uint64 InWorldId, TArrayView<const uint64> InDependencyIds);

	virtual uint32 GetChangeSerial() const override;

	virtual uint32 GetStreamingWorldCount() const override;
	virtual void EnumerateStreamingWorlds(TFunctionRef<void(const FStreamingWorldInfo&)> InCallback) const override;
	virtual void EnumerateStreamingWorldsAtTime(double InTime, TFunctionRef<void(const FStreamingWorldInfo&)> InCallback) const override;
	virtual bool ReadStreamingWorld(uint64 InWorldId, TFunctionRef<void(const FStreamingWorldInfo&)> InCallback) const override;

	virtual uint32 GetStreamingContainerCount(uint64 InWorldId) const override;
	virtual void EnumerateStreamingContainers(uint64 InWorldId, TFunctionRef<void(const FStreamingContainerInfo&)> InCallback) const override;
	virtual bool ReadStreamingContainer(uint64 InWorldId, uint64 InContainerId, TFunctionRef<void(const FStreamingContainerInfo&)> InCallback) const override;
	virtual void EnumerateChildStreamingContainers(uint64 InWorldId, uint64 InContainerId, TFunctionRef<void(const FStreamingContainerInfo&)> InCallback) const override;
	virtual void EnumerateRootStreamingContainers(uint64 InWorldId, TFunctionRef<void(const FStreamingContainerInfo&)> InCallback) const override;

	virtual void EnumerateStreamingContainerStateChanges(uint64 InWorldId, uint64 InContainerId, double InStartTime, double InEndTime, TFunctionRef<void(const FStreamingContainerStateChange&)> InCallback) const override;
	virtual EStreamingContainerState GetStreamingContainerStateAtTime(uint64 InWorldId, uint64 InContainerId, double InTime) const override;

	virtual float GetStreamingContainerPriorityAtTime(uint64 InWorldId, uint64 InContainerId, double InTime) const override;

	virtual void EnumerateStreamingSources(uint64 InWorldId, TFunctionRef<void(const FStreamingSourceInfo&)> InCallback) const override;
	virtual void EnumerateStreamingSourcesAtTime(uint64 InWorldId, double InTime, TFunctionRef<void(const FStreamingSourceInfo&)> InCallback) const override;
	virtual bool ReadStreamingSourceAtTime(uint64 InWorldId, uint64 InSourceId, double InTime, TFunctionRef<void(const FStreamingSourceInfo&)> InCallback) const override;

	virtual void EnumerateStreamingSourceUpdates(uint64 InWorldId, uint64 InSourceId, double InStartTime, double InEndTime, TFunctionRef<void(const FStreamingSourceUpdate&)> InCallback) const override;
	virtual FVector GetStreamingSourceLocationAtTime(uint64 InWorldId, uint64 InSourceId, double InTime) const override;

	virtual void EnumerateStreamingTagGroups(TFunctionRef<void(const FStreamingTagGroup&)> InCallback) const override;
	virtual bool ReadStreamingTagGroup(uint64 InGroupId, TFunctionRef<void(const FStreamingTagGroup&)> InCallback) const override;

	virtual void EnumerateStreamingTagsInGroup(uint64 InGroupId, TFunctionRef<void(const FStreamingTag&)> InCallback) const override;
	virtual bool ReadStreamingTag(uint64 InTagId, TFunctionRef<void(const FStreamingTag&)> InCallback) const override;

	virtual const TCHAR* GetPackageName(uint64 InPackageId) const override;

private:
	TraceServices::IAnalysisSession& Session;

	uint32 ChangeSerial = 0;

	TMap<uint64, FStreamingWorldInfo> Worlds;
	TMap<uint64, TMap<uint64, FStreamingContainerInfo> > ContainersPerWorld;
	TMap<uint64, TArray<FStreamingSourceInfo> > StreamingSourcesPerWorld;
	TMap<uint64, FStreamingTagGroup> TagGroups;
	TMap<uint64, FStreamingTag> Tags;
	TMap<uint64, TSet<uint64> > TagsPerGroup;

	TMap<uint64, TArray<FStreamingContainerStateChange> > ContainerStateChangesPerWorld;
	TMap<uint64, TArray<FStreamingSourceUpdate> > StreamingSourceUpdatesPerWorld;

	// Per world -> per container -> sorted array of priority updates
	TMap<uint64, TMap<uint64, TArray<FStreamingContainerPriorityUpdate> > > ContainerPrioritiesPerWorld;

	TMap<uint64, TMap<uint64, TSet<uint64> > > ContainerToChildContainersPerWorld;
	TMap<uint64, TSet<uint64> > RootContainersPerWorld;
	TMap<uint64, TMap<uint64, TArray<int32> > > ContainerToStateChangesPerWorld;
	TMap<uint64, TMap<uint64, TArray<int32> > > StreamingSourceToUpdatesPerWorld;

	TMap<uint64, TMap<uint64, int32> > ActiveStreamingSourceLookupPerWorld;
	TMap<uint64, TMap<uint64, const TCHAR*> > SourceNamesPerWorld;

	TMap<uint64, const TCHAR*> PackageNames;
};
