// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContainerMemoryEstimator.h"

#include "Common/ProviderLock.h"
#include "IWorldStreamingInsightsProvider.h"
#include "Templates/Function.h"
#include "WorldStreamingInsightsLog.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/AllocationsProvider.h"
#include "TraceServices/Model/Definitions.h"
#include "TraceServices/Model/MetadataProvider.h"
#include "TraceServices/Model/Strings.h"

////////////////////////////////////////////////////////////////////////////////////////////////////
// FAllocationsMemorySource
////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationsMemorySource::FAllocationsMemorySource(
	const TraceServices::IAllocationsProvider& InAllocationsProvider,
	const TraceServices::IMetadataProvider& InMetadataProvider,
	const TraceServices::IDefinitionProvider& InDefinitionProvider)
	: AllocationsProvider(InAllocationsProvider)
	, MetadataProvider(InMetadataProvider)
	, DefinitionProvider(InDefinitionProvider)
{
}

FAllocationsMemorySource::~FAllocationsMemorySource()
{
	CancelActiveQuery();
}

bool FAllocationsMemorySource::IsSnapshotReady() const
{
	return State == EState::SnapshotReady;
}

bool FAllocationsMemorySource::Tick(double InCurrentTime)
{
	const int64 CurrentBucket = FMath::RoundToInt64(InCurrentTime * (1000.0 / TimeQuantizationMs));

	// Scrubber moved into a new bucket while a query was in flight - abandon it so the next branch can start fresh against the new time.
	if (State == EState::SnapshotPending && CurrentBucket != LastQueriedBucket)
	{
		CancelActiveQuery();
	}

	if (State == EState::NoSnapshot || (State == EState::SnapshotReady && CurrentBucket != LastQueriedBucket))
	{
		StartQuery(InCurrentTime);
		LastQueriedBucket = CurrentBucket;
		return false;
	}

	if (State == EState::SnapshotPending)
	{
		ProcessResults();
		if (State == EState::SnapshotReady)
		{
			return true;
		}
	}

	return false;
}

const TMap<FString, FPackageMemoryBreakdown>& FAllocationsMemorySource::GetCachedSnapshot() const
{
	return CachedSnapshot;
}

void FAllocationsMemorySource::StartQuery(double InTime)
{
	if (!bMetadataResolved)
	{
		TraceServices::FProviderReadScopeLock MetaLock(MetadataProvider);
		AssetMetadataType = MetadataProvider.GetRegisteredMetadataType(FName(TEXT("Asset")));
		if (AssetMetadataType != TraceServices::IMetadataProvider::InvalidMetadataType)
		{
			AssetSchema = MetadataProvider.GetRegisteredMetadataSchema(AssetMetadataType);
			bMetadataResolved = true;
		}
	}

	if (AssetMetadataType == TraceServices::IMetadataProvider::InvalidMetadataType || !AssetSchema)
	{
		State = EState::SnapshotReady;
		CachedSnapshot.Reset();
		return;
	}

	TraceServices::FProviderReadScopeLock AllocLock(AllocationsProvider);

	TraceServices::IAllocationsProvider::FQueryParams Params;
	Params.Rule = TraceServices::IAllocationsProvider::EQueryRule::aAf;
	Params.TimeA = InTime;
	Params.TimeB = 0.0;
	Params.TimeC = 0.0;
	Params.TimeD = 0.0;

	ActiveQueryHandle = AllocationsProvider.StartQuery(Params);
	LastQueriedTime = InTime;
	PendingSnapshot.Reset();
	State = EState::SnapshotPending;
}

void FAllocationsMemorySource::ProcessResults()
{
	TraceServices::FProviderReadScopeLock AllocLock(AllocationsProvider);

	TraceServices::IAllocationsProvider::FQueryStatus Status = AllocationsProvider.PollQuery(ActiveQueryHandle);

	if (Status.Status == TraceServices::IAllocationsProvider::EQueryStatus::Available)
	{
		TraceServices::FProviderReadScopeLock MetaLock(MetadataProvider);
		TraceServices::FProviderReadScopeLock DefLock(DefinitionProvider);

		TraceServices::IAllocationsProvider::FQueryResult Result = Status.NextResult();
		while (Result.IsValid())
		{
			const uint32 Num = Result->Num();
			for (uint32 i = 0; i < Num; ++i)
			{
				const TraceServices::IAllocationsProvider::FAllocation* Alloc = Result->Get(i);
				if (!Alloc || Alloc->IsHeap())
				{
					continue;
				}

				const uint32 MetadataId = Alloc->GetMetadataId();
				if (MetadataId == TraceServices::IMetadataProvider::InvalidMetadataId)
				{
					continue;
				}

				MetadataProvider.EnumerateMetadata(
					Alloc->GetAllocThreadId(), MetadataId,
					[this, Alloc](uint32 StackDepth, uint16 Type, const void* Data, uint32 Size) -> bool
					{
						if (Type == AssetMetadataType)
						{
							const auto Reader = AssetSchema->Reader();
							const auto* PackageNameRef = Reader.GetValueAs<UE::Trace::FEventRef32>(static_cast<const uint8*>(Data), 2);
							const TraceServices::FStringDefinition* PackageDef = PackageNameRef ? DefinitionProvider.Get<TraceServices::FStringDefinition>(*PackageNameRef) : nullptr;

							if (PackageDef && PackageDef->Display)
							{
								FString PackageName(PackageDef->Display);
								FPackageMemoryBreakdown& Entry = PendingSnapshot.FindOrAdd(PackageName);
								Entry.TotalBytes += Alloc->GetSize();
							}

							return false;
						}
						return true;
					});
			}

			Result = Status.NextResult();
		}
	}

	if (Status.Status == TraceServices::IAllocationsProvider::EQueryStatus::Done)
	{
		CachedSnapshot = MoveTemp(PendingSnapshot);
		PendingSnapshot.Reset();
		State = EState::SnapshotReady;
	}
}

void FAllocationsMemorySource::CancelActiveQuery()
{
	if (State == EState::SnapshotPending)
	{
		TraceServices::FProviderReadScopeLock AllocLock(AllocationsProvider);
		AllocationsProvider.CancelQuery(ActiveQueryHandle);
		PendingSnapshot.Reset();
		State = EState::NoSnapshot;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FContainerMemoryEstimator
////////////////////////////////////////////////////////////////////////////////////////////////////

void FContainerMemoryEstimator::Init(
	const TraceServices::IAnalysisSession& InSession,
	const IWorldStreamingInsightsProvider& InStreamingProvider)
{
	Session = &InSession;
	StreamingProvider = &InStreamingProvider;

	const TraceServices::IAllocationsProvider* AllocationsProvider = TraceServices::ReadAllocationsProvider(InSession);
	const TraceServices::IMetadataProvider* MetadataProvider = TraceServices::ReadMetadataProvider(InSession);
	const TraceServices::IDefinitionProvider* DefinitionProvider = TraceServices::ReadDefinitionProvider(InSession);

	if (AllocationsProvider && MetadataProvider && DefinitionProvider)
	{
		MemorySource = MakeUnique<FAllocationsMemorySource>(*AllocationsProvider, *MetadataProvider, *DefinitionProvider);
	}
	else
	{
		UE_LOGF(LogWorldStreamingInsights, Log, "Memory source unavailable; capture with -trace=memalloc,metadata,assetmetadata to enable memory analysis.");
	}
}

void FContainerMemoryEstimator::Shutdown()
{
	MemorySource.Reset();
	Session = nullptr;
	StreamingProvider = nullptr;

	ContainerPackageSets.Empty();
	IncludedContainerIds.Empty();
	PackageRefCounts.Empty();
	ContainerMemoryResults.Empty();
}

void FContainerMemoryEstimator::Tick(double InCurrentTime, uint64 InWorldId)
{
	if (!Session || !StreamingProvider || !MemorySource)
	{
		return;
	}

	if (InWorldId != CurrentWorldId)
	{
		CurrentWorldId = InWorldId;
		bDependencyGraphDirty = true;
	}

	const uint32 ProviderSerial = StreamingProvider->GetChangeSerial();
	if (ProviderSerial != LastStreamingChangeSerial)
	{
		LastStreamingChangeSerial = ProviderSerial;
		bDependencyGraphDirty = true;
	}

	if (bDependencyGraphDirty)
	{
		BuildDependencyGraph();
		bDependencyGraphDirty = false;
		bRefCountsDirty = true;
		// BuildDependencyGraph cleared IncludedContainerIds; force the next bucket check to repopulate it so RecomputeRefCounts doesn't run on an empty set.
		LastInclusionBucket = INT64_MIN;
	}

	static constexpr double InclusionQuantizationMs = 100.0;
	const int64 CurrentBucket = FMath::RoundToInt64(InCurrentTime * (1000.0 / InclusionQuantizationMs));
	if (CurrentBucket != LastInclusionBucket)
	{
		LastInclusionBucket = CurrentBucket;
		if (UpdateIncludedContainers(InCurrentTime))
		{
			bRefCountsDirty = true;
		}
	}

	bool bNeedsRecompute = false;
	if (bRefCountsDirty)
	{
		RecomputeRefCounts();
		bRefCountsDirty = false;
		bNeedsRecompute = true;
	}

	const bool bSnapshotChanged = MemorySource->Tick(InCurrentTime);

	if ((bSnapshotChanged || bNeedsRecompute) && MemorySource->IsSnapshotReady())
	{
		ComputeContainerMemory();
	}
}

bool FContainerMemoryEstimator::IsReady() const
{
	return MemorySource && MemorySource->IsSnapshotReady() && !ContainerMemoryResults.IsEmpty();
}

const FContainerMemoryData* FContainerMemoryEstimator::GetContainerMemoryData(uint64 InContainerId) const
{
	return ContainerMemoryResults.Find(InContainerId);
}

void FContainerMemoryEstimator::BuildDependencyGraph()
{
	ContainerPackageSets.Reset();
	IncludedContainerIds.Reset();

	StreamingProvider->EnumerateStreamingContainers(CurrentWorldId, [this](const FStreamingContainerInfo& Container)
	{
		if (Container.DependencyPackageIds.IsEmpty())
		{
			return;
		}

		TSet<FString> DependencyNames;
		DependencyNames.Reserve(Container.DependencyPackageIds.Num());
		for (const uint64 DependencyId : Container.DependencyPackageIds)
		{
			const TCHAR* Name = StreamingProvider->GetPackageName(DependencyId);
			if (Name)
			{
				DependencyNames.Add(FString(Name));
			}
		}
		if (!DependencyNames.IsEmpty())
		{
			ContainerPackageSets.Add(Container.ContainerId, MoveTemp(DependencyNames));
		}
	});
}

bool FContainerMemoryEstimator::UpdateIncludedContainers(double InCurrentTime)
{
	TSet<uint64> PreviousIncluded = MoveTemp(IncludedContainerIds);
	IncludedContainerIds.Reset();

	StreamingProvider->EnumerateStreamingContainers(CurrentWorldId, [this, InCurrentTime](const FStreamingContainerInfo& Container)
	{
		if (!ContainerPackageSets.Contains(Container.ContainerId))
		{
			return;
		}

		EStreamingContainerState State = StreamingProvider->GetStreamingContainerStateAtTime(CurrentWorldId, Container.ContainerId, InCurrentTime);
		if (State != EStreamingContainerState::Unloaded)
		{
			IncludedContainerIds.Add(Container.ContainerId);
		}
	});

	return IncludedContainerIds.Num() != PreviousIncluded.Num() || !IncludedContainerIds.Includes(PreviousIncluded);
}

void FContainerMemoryEstimator::RecomputeRefCounts()
{
	PackageRefCounts.Reset();

	for (const uint64 ContainerId : IncludedContainerIds)
	{
		if (const TSet<FString>* DependencySet = ContainerPackageSets.Find(ContainerId))
		{
			for (const FString& PackageName : *DependencySet)
			{
				PackageRefCounts.FindOrAdd(PackageName)++;
			}
		}
	}
}

void FContainerMemoryEstimator::ComputeContainerMemory()
{
	ContainerMemoryResults.Reset();
	MaxContainerMemory = 0;
	MaxContainerUniqueMemory = 0;
	MaxContainerSharedMemory = 0;

	const TMap<FString, FPackageMemoryBreakdown>& Snapshot = MemorySource->GetCachedSnapshot();

	for (const uint64 ContainerId : IncludedContainerIds)
	{
		const TSet<FString>* DependencySetPtr = ContainerPackageSets.Find(ContainerId);
		if (!DependencySetPtr)
		{
			continue;
		}
		const TSet<FString>& DependencySet = *DependencySetPtr;

		FContainerMemoryData& ContainerData = ContainerMemoryResults.Add(ContainerId);
		ContainerData.TotalPackageCount = DependencySet.Num();

		for (const FString& PackageName : DependencySet)
		{
			const FPackageMemoryBreakdown* Breakdown = Snapshot.Find(PackageName);
			if (!Breakdown || Breakdown->TotalBytes <= 0)
			{
				continue;
			}

			const int32 RefCount = PackageRefCounts.FindRef(PackageName);
			const bool bIsUnique = (RefCount == 1);

			if (bIsUnique)
			{
				ContainerData.UniqueMemoryBytes += Breakdown->TotalBytes;
				ContainerData.UniquePackageCount++;
			}
			else
			{
				ContainerData.SharedMemoryBytes += Breakdown->TotalBytes;
				ContainerData.SharedPackageCount++;
			}
		}

		ContainerData.TotalMemoryBytes = ContainerData.UniqueMemoryBytes + ContainerData.SharedMemoryBytes;
		MaxContainerMemory = FMath::Max(MaxContainerMemory, ContainerData.TotalMemoryBytes);
		MaxContainerUniqueMemory = FMath::Max(MaxContainerUniqueMemory, ContainerData.UniqueMemoryBytes);
		MaxContainerSharedMemory = FMath::Max(MaxContainerSharedMemory, ContainerData.SharedMemoryBytes);
	}
}

bool FContainerMemoryEstimator::HasMemorySource() const
{
	return MemorySource != nullptr;
}

bool FContainerMemoryEstimator::IsMemorySourceReady() const
{
	return MemorySource && MemorySource->IsSnapshotReady();
}

bool FContainerMemoryEstimator::HasDependencyData() const
{
	return !ContainerPackageSets.IsEmpty();
}

bool FContainerMemoryEstimator::HasDependencyDataForContainer(uint64 InContainerId) const
{
	return ContainerPackageSets.Contains(InContainerId);
}

bool FContainerMemoryEstimator::IsContainerIncluded(uint64 InContainerId) const
{
	return IncludedContainerIds.Contains(InContainerId);
}

void FContainerMemoryEstimator::EnumerateContainerPackages(uint64 InContainerId, TFunctionRef<void(const FString&, int32)> InCallback) const
{
	if (const TSet<FString>* DependencySet = ContainerPackageSets.Find(InContainerId))
	{
		for (const FString& PackageName : *DependencySet)
		{
			const int32 RefCount = PackageRefCounts.FindRef(PackageName);
			if (RefCount > 0)
			{
				InCallback(PackageName, RefCount);
			}
		}
	}
}
