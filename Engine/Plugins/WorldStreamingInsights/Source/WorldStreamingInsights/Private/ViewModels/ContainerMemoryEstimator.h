// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Templates/FunctionFwd.h"
#include "TraceServices/Model/MetadataProvider.h"

class IWorldStreamingInsightsProvider;
struct FStreamingContainerInfo;

namespace TraceServices
{
	class IAnalysisSession;
	class IAllocationsProvider;
	class IDefinitionProvider;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Data structures
////////////////////////////////////////////////////////////////////////////////////////////////////

struct FPackageMemoryBreakdown
{
	int64 TotalBytes = 0;
};

struct FContainerMemoryData
{
	int64 UniqueMemoryBytes = 0;
	int64 SharedMemoryBytes = 0;
	int64 TotalMemoryBytes = 0;
	int32 UniquePackageCount = 0;
	int32 SharedPackageCount = 0;
	int32 TotalPackageCount = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// IContainerMemorySource - abstraction over where per-package memory data comes from
////////////////////////////////////////////////////////////////////////////////////////////////////

class IContainerMemorySource
{
public:
	virtual ~IContainerMemorySource() = default;

	// Returns true when the snapshot is ready for reading.
	virtual bool IsSnapshotReady() const = 0;

	// Drive the async state machine. Call each tick.
	// Returns true if the snapshot changed this tick.
	virtual bool Tick(double InCurrentTime) = 0;

	// Pre-computed snapshot: package name -> memory breakdown.
	virtual const TMap<FString, FPackageMemoryBreakdown>& GetCachedSnapshot() const = 0;

	// The actual time used for the last completed query.
	virtual double GetLastQueriedTime() const = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FAllocationsMemorySource - queries IAllocationsProvider, resolves Package metadata
////////////////////////////////////////////////////////////////////////////////////////////////////

class FAllocationsMemorySource : public IContainerMemorySource
{
public:
	FAllocationsMemorySource(
		const TraceServices::IAllocationsProvider& InAllocationsProvider,
		const TraceServices::IMetadataProvider& InMetadataProvider,
		const TraceServices::IDefinitionProvider& InDefinitionProvider);
	~FAllocationsMemorySource();

	virtual bool IsSnapshotReady() const override;
	virtual bool Tick(double InCurrentTime) override;
	virtual const TMap<FString, FPackageMemoryBreakdown>& GetCachedSnapshot() const override;
	virtual double GetLastQueriedTime() const override { return LastQueriedTime; }

private:
	enum class EState : uint8
	{
		NoSnapshot,
		SnapshotPending,
		SnapshotReady
	};

	void StartQuery(double InTime);
	void ProcessResults();
	void CancelActiveQuery();

	const TraceServices::IAllocationsProvider& AllocationsProvider;
	const TraceServices::IMetadataProvider& MetadataProvider;
	const TraceServices::IDefinitionProvider& DefinitionProvider;

	EState State = EState::NoSnapshot;
	uint64 ActiveQueryHandle = 0;

	// Resolved metadata type for "Asset" - cached once.
	uint16 AssetMetadataType = TraceServices::IMetadataProvider::InvalidMetadataType;
	const TraceServices::FMetadataSchema* AssetSchema = nullptr;
	bool bMetadataResolved = false;

	// Time quantization: only re-query when quantized bucket changes.
	static constexpr double TimeQuantizationMs = 100.0;
	int64 LastQueriedBucket = INT64_MIN;
	double LastQueriedTime = 0.0;

	TMap<FString, FPackageMemoryBreakdown> CachedSnapshot;
	TMap<FString, FPackageMemoryBreakdown> PendingSnapshot;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FContainerMemoryEstimator - orchestrates dependency graph + memory source + per-container aggregation
////////////////////////////////////////////////////////////////////////////////////////////////////

class FContainerMemoryEstimator
{
public:
	void Init(
		const TraceServices::IAnalysisSession& InSession,
		const IWorldStreamingInsightsProvider& InStreamingProvider);

	void Shutdown();

	// Drive the estimator. Call each tick with the current scrubber time and world ID.
	// Caller must hold FAnalysisSessionReadScope.
	void Tick(double InCurrentTime, uint64 InWorldId);

	// Results - valid after Tick() when IsReady() returns true.
	bool IsReady() const;
	const FContainerMemoryData* GetContainerMemoryData(uint64 InContainerId) const;
	int64 GetMaxContainerMemory() const { return MaxContainerMemory; }
	int64 GetMaxContainerUniqueMemory() const { return MaxContainerUniqueMemory; }
	int64 GetMaxContainerSharedMemory() const { return MaxContainerSharedMemory; }

	double GetLastQueriedTime() const { return MemorySource ? MemorySource->GetLastQueriedTime() : 0.0; }
	void EnumerateContainerPackages(uint64 InContainerId, TFunctionRef<void(const FString& PackageName, int32 RefCount)> InCallback) const;

	// Diagnostic state - for UI messaging when memory data is unavailable.
	bool HasMemorySource() const;
	bool IsMemorySourceReady() const;
	bool HasDependencyData() const;
	bool HasDependencyDataForContainer(uint64 InContainerId) const;
	bool IsContainerIncluded(uint64 InContainerId) const;

private:
	void BuildDependencyGraph();
	bool UpdateIncludedContainers(double InCurrentTime);
	void RecomputeRefCounts();
	void ComputeContainerMemory();

	const TraceServices::IAnalysisSession* Session = nullptr;
	const IWorldStreamingInsightsProvider* StreamingProvider = nullptr;

	TUniquePtr<IContainerMemorySource> MemorySource;

	uint64 CurrentWorldId = 0;
	uint32 LastStreamingChangeSerial = 0;
	bool bDependencyGraphDirty = true;
	bool bRefCountsDirty = true;
	int64 LastInclusionBucket = INT64_MIN;

	// Rebuilt from provider topology when streaming data changes; refcounts re-aggregated when the included-container set changes.
	TMap<uint64, TSet<FString>> ContainerPackageSets;
	TSet<uint64> IncludedContainerIds;
	TMap<FString, int32> PackageRefCounts;

	TMap<uint64, FContainerMemoryData> ContainerMemoryResults;
	int64 MaxContainerMemory = 0;
	int64 MaxContainerUniqueMemory = 0;
	int64 MaxContainerSharedMemory = 0;
};
