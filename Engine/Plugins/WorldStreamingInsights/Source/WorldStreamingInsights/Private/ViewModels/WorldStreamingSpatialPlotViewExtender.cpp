// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/WorldStreamingSpatialPlotViewExtender.h"

#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IWorldStreamingInsightsProvider.h"
#include "WorldStreamingInsightsLog.h"
#include "Modules/ModuleManager.h"
#include "Insights/IUnrealInsightsModule.h"
#include "Insights/Common/ColorMapUtils.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SWorldStreamingFilterPanel.h"

#define LOCTEXT_NAMESPACE "WorldStreamingSpatialPlotViewExtender"

struct FMemoryDrilldownSource
{
	FText Label;
	double QueryTime = 0.0;
	int64 TotalBytes = 0;
	int64 UniqueBytes = 0;
	int64 SharedBytes = 0;
	TSet<FString> AllPackages;
	TSet<FString> UniquePackages;
	TSet<FString> SharedPackages;
};

static void PopulateDrilldownPackages(FMemoryDrilldownSource& OutSource, const FContainerMemoryEstimator& InEstimator, uint64 InContainerId)
{
	InEstimator.EnumerateContainerPackages(InContainerId, [&OutSource](const FString& PackageName, int32 RefCount)
	{
		OutSource.AllPackages.Add(PackageName);
		if (RefCount == 1)
		{
			OutSource.UniquePackages.Add(PackageName);
		}
		else
		{
			OutSource.SharedPackages.Add(PackageName);
		}
	});
}

static void AddMemoryProfilerDrilldown(FMenuBuilder& InOutMenuBuilder, FMemoryDrilldownSource InSource)
{
	InOutMenuBuilder.AddSubMenu(
		LOCTEXT("OpenInMemoryProfiler", "Open in Memory Profiler"),
		LOCTEXT("OpenInMemoryProfilerTooltip", "Open the Memory Profiler filtered to these packages"),
		FNewMenuDelegate::CreateLambda([Source = MoveTemp(InSource)](FMenuBuilder& InOutSubMenuBuilder)
		{
			auto OpenProfilerWithFilter = [](double InTime, const TSet<FString>& InPackages, const FText& InTabName)
			{
				IUnrealInsightsModule& Module = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
				Module.OpenMemoryAllocationsView(InTime, InTabName, InPackages);
			};

			const double QueryTime = Source.QueryTime;
			const FText Label = Source.Label;

			InOutSubMenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("OpenAll", "All ({0})"), FText::FromString(FString::Printf(TEXT("%.1f MiB"), static_cast<double>(Source.TotalBytes) / (1024.0 * 1024.0)))),
				LOCTEXT("OpenAllTooltip", "All packages this container depends on"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([Packages = Source.AllPackages, QueryTime, Label, OpenProfilerWithFilter]()
				{
					const FText TabName = FText::Format(LOCTEXT("AllocsTabAll", "Allocs: {0}"), Label);
					OpenProfilerWithFilter(QueryTime, Packages, TabName);
				})));

			InOutSubMenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("OpenUnique", "Unique ({0})"), FText::FromString(FString::Printf(TEXT("%.1f MiB"), static_cast<double>(Source.UniqueBytes) / (1024.0 * 1024.0)))),
				LOCTEXT("OpenUniqueTooltip", "Packages referenced only by this container"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([Packages = Source.UniquePackages, QueryTime, Label, OpenProfilerWithFilter]()
				{
					const FText TabName = FText::Format(LOCTEXT("AllocsTabUnique", "Allocs: {0} (Unique)"), Label);
					OpenProfilerWithFilter(QueryTime, Packages, TabName);
				})));

			InOutSubMenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("OpenShared", "Shared ({0})"), FText::FromString(FString::Printf(TEXT("%.1f MiB"), static_cast<double>(Source.SharedBytes) / (1024.0 * 1024.0)))),
				LOCTEXT("OpenSharedTooltip", "Packages referenced by multiple containers"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([Packages = Source.SharedPackages, QueryTime, Label, OpenProfilerWithFilter]()
				{
					const FText TabName = FText::Format(LOCTEXT("AllocsTabShared", "Allocs: {0} (Shared)"), Label);
					OpenProfilerWithFilter(QueryTime, Packages, TabName);
				})));
		}));
}

static FLinearColor GetBaseColorForContainerState(EStreamingContainerState InState)
{
	switch (InState)
	{
	case EStreamingContainerState::Unloaded:
		return FLinearColor(FColor(128, 128, 128)); // Gray
	case EStreamingContainerState::Loading:
		return FLinearColor(FColor(255, 242, 102)); // Warm yellow
	case EStreamingContainerState::Loaded:
		return FLinearColor(FColor(100, 148, 237)); // Cornflower blue
	case EStreamingContainerState::Activating:
		return FLinearColor(FColor(230, 140,   0)); // Deep amber
	case EStreamingContainerState::Active:
		return FLinearColor(FColor(  0, 166, 153)); // Teal
	case EStreamingContainerState::Deactivating:
		return FLinearColor(FColor(204,  89, 140)); // Reddish-purple
	}

	ensureMsgf(false, TEXT("Unhandled EStreamingContainerState: %d"), static_cast<int>(InState));
	return FLinearColor::Black;
}

static FLinearColor GetHeatmapColor(float InNormalizedValue, EStreamingPalette InPalette)
{
	// Floor-remap so the darkest cells remain visible at 0.5 opacity on dark backgrounds.
	// Grayscale needs a higher floor than colormaps because it has no hue contrast to help.
	const float Floor = (InPalette == EStreamingPalette::Grayscale) ? 0.35f : 0.2f;
	const float FlooredValue = Floor + FMath::Clamp(InNormalizedValue, 0.0f, 1.0f) * (1.0f - Floor);

	switch (InPalette)
	{
	case EStreamingPalette::Viridis:
		return UE::Insights::ColorMapViridis(FlooredValue);
	case EStreamingPalette::Inferno:
		return UE::Insights::ColorMapInferno(FlooredValue);
	case EStreamingPalette::Grayscale:
		return UE::Insights::ColorMapGrayscale(FlooredValue);
	}

	ensureMsgf(false, TEXT("Unhandled EStreamingPalette: %d"), static_cast<int>(InPalette));
	return UE::Insights::ColorMapViridis(FlooredValue);
}

static FString GetContainerStateName(EStreamingContainerState InState)
{
	switch (InState)
	{
	case EStreamingContainerState::Unloaded:
		return TEXT("Unloaded");
	case EStreamingContainerState::Loading:
		return TEXT("Loading");
	case EStreamingContainerState::Loaded:
		return TEXT("Loaded");
	case EStreamingContainerState::Activating:
		return TEXT("Activating");
	case EStreamingContainerState::Active:
		return TEXT("Active");
	case EStreamingContainerState::Deactivating:
		return TEXT("Deactivating");
	}

	ensureMsgf(false, TEXT("Unhandled EStreamingContainerState: %d"), static_cast<int>(InState));
	return TEXT("Unknown");
}

void FWorldStreamingSpatialPlotViewExtender::OnBeginSession(const TraceServices::IAnalysisSession& InAnalysisSession)
{
	AnalysisSession = &InAnalysisSession;
	WorldStreamingInsightsProvider = ReadWorldStreamingInsightsProvider(InAnalysisSession);

	ContainerVisibilityOverrides.Empty();
	TagVisibilityOverrides.Empty();
	UntaggedVisibilityOverrides.Empty();
	NewTagDefaultPerGroup.Empty();
	ResolvedContainerVisibility.Empty();
	ResolvedTagVisibility.Empty();
	FilterChangeSerial = 0;
	LastResolvedFilterChangeSerial = UINT32_MAX;
	LastResolvedProviderChangeSerial = UINT32_MAX;
	VisualizationMode = EStreamingVisualizationMode::State;
	Palette = EStreamingPalette::Viridis;

	CachedWorldId = UINT64_MAX;
	LastValidWorldId = UINT64_MAX;
	CachedCurrentTraceTime = 0.0;
	bHasPriorityData = false;

	if (WorldStreamingInsightsProvider)
	{
		MemoryEstimator.Init(InAnalysisSession, *WorldStreamingInsightsProvider);
	}
}

void FWorldStreamingSpatialPlotViewExtender::OnEndSession()
{
	MemoryEstimator.Shutdown();

	AnalysisSession = nullptr;
	WorldStreamingInsightsProvider = nullptr;

	ContainerVisibilityOverrides.Empty();
	TagVisibilityOverrides.Empty();
	UntaggedVisibilityOverrides.Empty();
	NewTagDefaultPerGroup.Empty();
	ResolvedContainerVisibility.Empty();
	ResolvedTagVisibility.Empty();
	FilterChangeSerial = 0;
	LastResolvedFilterChangeSerial = UINT32_MAX;
	LastResolvedProviderChangeSerial = UINT32_MAX;
	VisualizationMode = EStreamingVisualizationMode::State;
	Palette = EStreamingPalette::Viridis;
	CachedWorldId = UINT64_MAX;
	LastValidWorldId = UINT64_MAX;
	CachedCurrentTraceTime = 0.0;
	bHasPriorityData = false;
}

FName FWorldStreamingSpatialPlotViewExtender::GetLayerName() const
{
	return FName("WorldStreaming");
}

FText FWorldStreamingSpatialPlotViewExtender::GetLayerDisplayName() const
{
	return LOCTEXT("WorldStreamingLayerDisplayName", "World Streaming");
}

void FWorldStreamingSpatialPlotViewExtender::EnumerateRegions(TFunctionRef<void(const UE::Insights::SpatialProfiler::FSpatialPlotRegion&)> InCallback)
{
	if (WorldStreamingInsightsProvider && AnalysisSession)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);

		if (CachedWorldId != UINT64_MAX)
		{
			TArray<uint64> KnownGroupIds;

			WorldStreamingInsightsProvider->EnumerateStreamingTagGroups([&KnownGroupIds](const FStreamingTagGroup& TagGroup)
			{
				KnownGroupIds.Add(TagGroup.GroupId);
			});

			WorldStreamingInsightsProvider->EnumerateStreamingContainers(CachedWorldId, [this, &KnownGroupIds, &InCallback](const FStreamingContainerInfo& ContainerInfo)
			{
				if (!ContainerInfo.Bounds.IsSet())
				{
					return;
				}

				bool bPassesContainerFilter = IsContainerVisible(ContainerInfo.ContainerId);
				bool bPassesTagFilter = PassesTagFilter(ContainerInfo.Tags, KnownGroupIds);

				if (bPassesContainerFilter && bPassesTagFilter)
				{
					UE::Insights::SpatialProfiler::FSpatialPlotRegion Region;
					Region.Id = ContainerInfo.ContainerId;
					Region.Label = FText::FromString(ContainerInfo.Name);
					const FBox& ContainerBounds = ContainerInfo.Bounds.GetValue();
					Region.Bounds = FBox2D(FVector2D(ContainerBounds.Min.X, ContainerBounds.Min.Y), FVector2D(ContainerBounds.Max.X, ContainerBounds.Max.Y));

					if (VisualizationMode == EStreamingVisualizationMode::State)
					{
						EStreamingContainerState State = WorldStreamingInsightsProvider->GetStreamingContainerStateAtTime(CachedWorldId, ContainerInfo.ContainerId, CachedCurrentTraceTime);
						Region.BorderColor = GetBaseColorForContainerState(State);
						Region.FillColor = Region.BorderColor.CopyWithNewOpacity(0.7f);
					}
					else if (IsMemoryVisualizationMode(VisualizationMode))
					{
						if (const FContainerMemoryData* ContainerData = MemoryEstimator.GetContainerMemoryData(ContainerInfo.ContainerId))
						{
							int64 DisplayValue = 0;
							switch (VisualizationMode)
							{
							case EStreamingVisualizationMode::MemoryTotal:
								DisplayValue = ContainerData->TotalMemoryBytes;
								break;
							case EStreamingVisualizationMode::MemoryUnique:
								DisplayValue = ContainerData->UniqueMemoryBytes;
								break;
							case EStreamingVisualizationMode::MemoryShared:
								DisplayValue = ContainerData->SharedMemoryBytes;
								break;
							default:
								break;
							}

							const int64 MaxValue = (bFixedScale && FixedScaleMaxMiB > 0)
								? static_cast<int64>(FixedScaleMaxMiB) * 1024 * 1024
								: GetAutoScaleMax(VisualizationMode) * static_cast<int64>(1024 * 1024);
							const float Normalized = (MaxValue > 0) ? FMath::Clamp(static_cast<float>(DisplayValue) / static_cast<float>(MaxValue), 0.0f, 1.0f) : 0.0f;
							Region.BorderColor = GetHeatmapColor(Normalized, Palette);
							Region.FillColor = Region.BorderColor.CopyWithNewOpacity(0.7f);

							const double MiB = static_cast<double>(ContainerData->TotalMemoryBytes) / (1024.0 * 1024.0);
							Region.Label = FText::FromString(FString::Printf(TEXT("%s - %.1f MiB"), ContainerInfo.Name, MiB));
						}
						else
						{
							Region.BorderColor = FLinearColor(0.3f, 0.3f, 0.3f, 1.0f);
							Region.FillColor = FLinearColor(0.3f, 0.3f, 0.3f, 0.3f);
						}
					}
					else
					{
						EStreamingContainerState State = WorldStreamingInsightsProvider->GetStreamingContainerStateAtTime(CachedWorldId, ContainerInfo.ContainerId, CachedCurrentTraceTime);
						float Priority = (State == EStreamingContainerState::Unloaded)
							? -1.0f
							: WorldStreamingInsightsProvider->GetStreamingContainerPriorityAtTime(CachedWorldId, ContainerInfo.ContainerId, CachedCurrentTraceTime);

						if (Priority < 0.0f)
						{
							Region.BorderColor = FLinearColor(0.2f, 0.2f, 0.2f, 0.3f);
							Region.FillColor = Region.BorderColor.CopyWithNewOpacity(0.3f);
						}
						else
						{
							bHasPriorityData = true;
							const float PriorityGradient = FMath::Cube(1.0f - FMath::Clamp(Priority, 0.0f, 1.0f));
							Region.BorderColor = GetHeatmapColor(PriorityGradient, Palette);
							Region.FillColor = Region.BorderColor.CopyWithNewOpacity(0.7f);
						}
					}

					InCallback(Region);
				}
			});
		}
	}
}

void FWorldStreamingSpatialPlotViewExtender::EnumerateMarkers(TFunctionRef<void(const UE::Insights::SpatialProfiler::FSpatialPlotMarker&)> InCallback)
{
	if (WorldStreamingInsightsProvider && AnalysisSession)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);

		if (CachedWorldId != UINT64_MAX)
		{
			WorldStreamingInsightsProvider->EnumerateStreamingSourcesAtTime(CachedWorldId, CachedCurrentTraceTime, [this, &InCallback](const FStreamingSourceInfo& StreamingSourceInfo)
			{
				FVector Location = WorldStreamingInsightsProvider->GetStreamingSourceLocationAtTime(CachedWorldId, StreamingSourceInfo.SourceId, CachedCurrentTraceTime);

				UE::Insights::SpatialProfiler::FSpatialPlotMarker Marker;
				Marker.Id = StreamingSourceInfo.SourceId;
				Marker.Label = FText::FromString(StreamingSourceInfo.Name);
				Marker.Color = FLinearColor::White;
				Marker.Position = FVector2D(Location.X, Location.Y);

				InCallback(Marker);
			});
		}
	}
}

TOptional<UE::Insights::SpatialProfiler::FSpatialPlotLegend> FWorldStreamingSpatialPlotViewExtender::GetLegend() const
{
	using namespace UE::Insights::SpatialProfiler;

	if (CachedWorldId == UINT64_MAX)
	{
		return {};
	}

	FSpatialPlotLegend Legend;

	if (VisualizationMode == EStreamingVisualizationMode::State)
	{
		Legend.Title = LOCTEXT("LegendState", "Container State");
		Legend.Type = ESpatialPlotLegendType::Discrete;

		const EStreamingContainerState States[] = {
			EStreamingContainerState::Active,
			EStreamingContainerState::Activating,
			EStreamingContainerState::Loaded,
			EStreamingContainerState::Loading,
			EStreamingContainerState::Deactivating,
			EStreamingContainerState::Unloaded
		};
		for (EStreamingContainerState State : States)
		{
			FSpatialPlotLegendEntry Entry;
			Entry.Color = GetBaseColorForContainerState(State);
			Entry.Label = FText::FromString(GetContainerStateName(State));
			Legend.Entries.Add(MoveTemp(Entry));
		}
	}
	else if (VisualizationMode == EStreamingVisualizationMode::Priority)
	{
		Legend.Title = LOCTEXT("LegendPriority", "Priority");
		Legend.Type = ESpatialPlotLegendType::Gradient;

		// Priority uses cube mapping (PriorityGradient = Priority^3); legend samples at evenly-spaced T values.
		constexpr int32 NumStops = 8;
		for (int32 StopIndex = 0; StopIndex < NumStops; ++StopIndex)
		{
			const float T = 1.0f - static_cast<float>(StopIndex) / static_cast<float>(NumStops - 1);
			FSpatialPlotLegendEntry Entry;
			Entry.Color = GetHeatmapColor(T, Palette);
			if (StopIndex == 0)
			{
				Entry.Label = LOCTEXT("LegendPriorityHigh", "High");
			}
			else if (StopIndex == NumStops - 1)
			{
				Entry.Label = LOCTEXT("LegendPriorityLow", "Low");
			}
			Legend.Entries.Add(MoveTemp(Entry));
		}
	}
	else
	{
		if (!MemoryEstimator.IsReady())
		{
			return {};
		}

		switch (VisualizationMode)
		{
		case EStreamingVisualizationMode::MemoryTotal:
			Legend.Title = LOCTEXT("LegendMemoryTotal", "Memory (Total)");
			break;
		case EStreamingVisualizationMode::MemoryUnique:
			Legend.Title = LOCTEXT("LegendMemoryUnique", "Memory (Unique)");
			break;
		case EStreamingVisualizationMode::MemoryShared:
			Legend.Title = LOCTEXT("LegendMemoryShared", "Memory (Shared)");
			break;
		default:
			Legend.Title = LOCTEXT("LegendMemory", "Memory");
			break;
		}
		Legend.Type = ESpatialPlotLegendType::Gradient;

		const int32 MaxMiBInt = (bFixedScale && FixedScaleMaxMiB > 0)
			? FixedScaleMaxMiB
			: GetAutoScaleMax(VisualizationMode);
		const double MaxMiB = static_cast<double>(MaxMiBInt);

		// Entries[0] = top of bar = max value (bright). Entries[N-1] = bottom = 0 (dark).
		constexpr int32 NumStops = 8;
		for (int32 StopIndex = 0; StopIndex < NumStops; ++StopIndex)
		{
			const float T = 1.0f - static_cast<float>(StopIndex) / static_cast<float>(NumStops - 1);
			FSpatialPlotLegendEntry Entry;
			Entry.Color = GetHeatmapColor(T, Palette);
			if (StopIndex == 0)
			{
				Entry.Label = FText::FromString(FString::Printf(TEXT("%.0f MiB"), MaxMiB));
			}
			else if (StopIndex == NumStops / 2)
			{
				Entry.Label = FText::FromString(FString::Printf(TEXT("%.0f MiB"), MaxMiB * 0.5));
			}
			else if (StopIndex == NumStops - 1)
			{
				Entry.Label = LOCTEXT("LegendMemoryMin", "0 MiB");
			}
			Legend.Entries.Add(MoveTemp(Entry));
		}
	}

	return Legend;
}

bool FWorldStreamingSpatialPlotViewExtender::AppendTooltip(FTooltipDrawState& InOutTooltip, const UE::Insights::SpatialProfiler::FSpatialPlotHitTestResult& InHitTestResult) const
{
	if (!AnalysisSession)
	{
		return false;
	}

	if (!WorldStreamingInsightsProvider)
	{
		return false;
	}

	if (CachedWorldId == UINT64_MAX)
	{
		return false;
	}

	bool bAddedContent = false;
	TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
	for (const UE::Insights::SpatialProfiler::FSpatialPlotMarker& Marker : InHitTestResult.Markers)
	{
		WorldStreamingInsightsProvider->ReadStreamingSourceAtTime(CachedWorldId, Marker.Id, CachedCurrentTraceTime, [&InOutTooltip, &Marker, &bAddedContent](const FStreamingSourceInfo& SourceInfo)
		{
			InOutTooltip.AddTitle(SourceInfo.Name);
			InOutTooltip.AddNameValueTextLine(LOCTEXT("StreamingSourcePosition", "Position").ToString(), Marker.Position.ToString());
			bAddedContent = true;
		});
	}

	for (const UE::Insights::SpatialProfiler::FSpatialPlotRegion& Region : InHitTestResult.Regions)
	{
		WorldStreamingInsightsProvider->ReadStreamingContainer(CachedWorldId, Region.Id, [this, &InOutTooltip, &bAddedContent](const FStreamingContainerInfo& ContainerInfo)
			{
				InOutTooltip.AddTitle(ContainerInfo.Name);

				if (VisualizationMode == EStreamingVisualizationMode::Priority)
				{
					float Priority = WorldStreamingInsightsProvider->GetStreamingContainerPriorityAtTime(CachedWorldId, ContainerInfo.ContainerId, CachedCurrentTraceTime);
					FString PriorityString = Priority < 0.0f ? TEXT("N/A") : FString::Printf(TEXT("%.4f"), Priority);
					InOutTooltip.AddNameValueTextLine(LOCTEXT("StreamingContainerPriority", "Priority").ToString(), PriorityString);
				}

				EStreamingContainerState CurrentState = WorldStreamingInsightsProvider->GetStreamingContainerStateAtTime(CachedWorldId, ContainerInfo.ContainerId, CachedCurrentTraceTime);
				InOutTooltip.AddNameValueTextLine(LOCTEXT("StreamingContainerState", "State").ToString(), GetContainerStateName(CurrentState));
				if (ContainerInfo.Bounds.IsSet())
				{
					const FBox& ContainerBounds = ContainerInfo.Bounds.GetValue();
					FString BoundsString = FString::Printf(TEXT("Min:(%.2f, %.2f) Max(%.2f, %.2f)"), ContainerBounds.Min.X, ContainerBounds.Min.Y, ContainerBounds.Max.X, ContainerBounds.Max.Y);
					InOutTooltip.AddNameValueTextLine(LOCTEXT("StreamingContainerBounds", "Bounds").ToString(), BoundsString);
				}

				TMap<uint64, TArray<FString> > TagsByGroup;
				for (uint64 Tag : ContainerInfo.Tags)
				{
					WorldStreamingInsightsProvider->ReadStreamingTag(Tag, [&TagsByGroup](const FStreamingTag& StreamingTag)
					{
						TagsByGroup.FindOrAdd(StreamingTag.GroupId).Add(StreamingTag.Name);
					});
				}

				for (const TPair<uint64, TArray<FString> >& Entry : TagsByGroup)
				{
					FString GroupName;
					WorldStreamingInsightsProvider->ReadStreamingTagGroup(Entry.Key, [&GroupName](const FStreamingTagGroup& Group)
					{
						GroupName = Group.Name;
					});
					FString TagsString = FString::Join(Entry.Value, TEXT(", "));
					InOutTooltip.AddNameValueTextLine(GroupName, TagsString);
				}

				if (IsMemoryVisualizationMode(VisualizationMode))
				{
					static const FString MemoryNotAvailableLabel = LOCTEXT("MemoryNotAvailable", "Memory").ToString();
					static const FString MemoryDataNotAvailableValue = LOCTEXT("MemoryDataNotAvailableValue", "Allocation data or dependency data not available - recapture with -trace=WorldStreaming,WorldStreamingDependencies,memalloc,metadata,assetmetadata").ToString();

					if (const FContainerMemoryData* ContainerData = MemoryEstimator.GetContainerMemoryData(ContainerInfo.ContainerId))
					{
						InOutTooltip.AddNameValueTextLine(
							LOCTEXT("MemoryTotal", "Memory (Total)").ToString(),
							FString::Printf(TEXT("%.2f MiB"), static_cast<double>(ContainerData->TotalMemoryBytes) / (1024.0 * 1024.0)));
						InOutTooltip.AddNameValueTextLine(
							LOCTEXT("MemoryUnique", "Memory (Unique)").ToString(),
							FString::Printf(TEXT("%.2f MiB (%d packages)"), static_cast<double>(ContainerData->UniqueMemoryBytes) / (1024.0 * 1024.0), ContainerData->UniquePackageCount));
						InOutTooltip.AddNameValueTextLine(
							LOCTEXT("MemoryShared", "Memory (Shared)").ToString(),
							FString::Printf(TEXT("%.2f MiB (%d packages)"), static_cast<double>(ContainerData->SharedMemoryBytes) / (1024.0 * 1024.0), ContainerData->SharedPackageCount));
						if (const FContainerMemoryData* PersistentLevelData = MemoryEstimator.GetContainerMemoryData(CachedWorldId))
						{
							InOutTooltip.AddNameValueTextLine(
								LOCTEXT("PersistentLevelMemory", "Persistent Level").ToString(),
								FString::Printf(TEXT("%.2f MiB"), static_cast<double>(PersistentLevelData->TotalMemoryBytes) / (1024.0 * 1024.0)));
						}
					}
					else if (!MemoryEstimator.HasMemorySource())
					{
						InOutTooltip.AddNameValueTextLine(MemoryNotAvailableLabel, MemoryDataNotAvailableValue);
					}
					else if (!MemoryEstimator.IsMemorySourceReady())
					{
						InOutTooltip.AddNameValueTextLine(
							LOCTEXT("MemoryComputing", "Memory").ToString(),
							LOCTEXT("MemoryComputingValue", "Computing...").ToString());
					}
					else if (!MemoryEstimator.HasDependencyData())
					{
						InOutTooltip.AddNameValueTextLine(MemoryNotAvailableLabel, MemoryDataNotAvailableValue);
					}
					else if (!MemoryEstimator.HasDependencyDataForContainer(ContainerInfo.ContainerId))
					{
						InOutTooltip.AddNameValueTextLine(
							LOCTEXT("MemoryNoContainerDependencies", "Memory").ToString(),
							LOCTEXT("MemoryNoContainerDependenciesValue", "No dependency data for this container - memory cannot be attributed").ToString());
					}
					else if (!MemoryEstimator.IsContainerIncluded(ContainerInfo.ContainerId))
					{
						InOutTooltip.AddNameValueTextLine(
							LOCTEXT("MemoryContainerUnloaded", "Memory").ToString(),
							LOCTEXT("MemoryContainerUnloadedValue", "Container is unloaded - no memory attributed").ToString());
					}
				}

				bAddedContent = true;
			});
	}

	return bAddedContent;
}

bool FWorldStreamingSpatialPlotViewExtender::ExtendContextMenu(FMenuBuilder& InOutMenuBuilder, const UE::Insights::SpatialProfiler::FSpatialPlotHitTestResult& InHitTestResult) const
{
	if (!AnalysisSession)
	{
		return false;
	}

	if (!WorldStreamingInsightsProvider)
	{
		return false;
	}

	if (CachedWorldId == UINT64_MAX)
	{
		return false;
	}

	bool bAddedContent = false;
	TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);

	for (const UE::Insights::SpatialProfiler::FSpatialPlotMarker& Marker : InHitTestResult.Markers)
	{
		WorldStreamingInsightsProvider->ReadStreamingSourceAtTime(CachedWorldId, Marker.Id, CachedCurrentTraceTime, [&InOutMenuBuilder, &Marker, &bAddedContent](const FStreamingSourceInfo& SourceInfo)
		{
			InOutMenuBuilder.BeginSection(NAME_None, FText::Format(LOCTEXT("StreamingSourceSection", "Streaming Source: {0}"), Marker.Label));
			InOutMenuBuilder.AddMenuEntry(
				LOCTEXT("CopyStreamingSourceNameToClipboard", "Copy Name To Clipboard"),
				LOCTEXT("CopyStreamingSourceNameToClipboardTooltip", "Copy Streaming Source name into the clipboard"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([ClipboardText=FString(SourceInfo.Name)]() { FPlatformApplicationMisc::ClipboardCopy(*ClipboardText); })),
				NAME_None,
				EUserInterfaceActionType::Button
			);
			InOutMenuBuilder.EndSection();

			bAddedContent = true;
		});
	}

	if (IsMemoryVisualizationMode(VisualizationMode))
	{
		const bool bMemoryReady = MemoryEstimator.IsMemorySourceReady();
		const FContainerMemoryData* PersistentLevelData = (bMemoryReady && CachedWorldId != UINT64_MAX) ? MemoryEstimator.GetContainerMemoryData(CachedWorldId) : nullptr;

		FText SectionTitle = PersistentLevelData
			? FText::Format(LOCTEXT("PersistentLevelSectionWithSize", "Persistent Level ({0})"), FText::FromString(FString::Printf(TEXT("%.1f MiB"), static_cast<double>(PersistentLevelData->TotalMemoryBytes) / (1024.0 * 1024.0))))
			: LOCTEXT("PersistentLevelSection", "Persistent Level");
		InOutMenuBuilder.BeginSection(NAME_None, SectionTitle);

		if (PersistentLevelData)
		{
			FMemoryDrilldownSource Source;
			Source.Label = LOCTEXT("PersistentLevelLabel", "Persistent Level");
			Source.QueryTime   = MemoryEstimator.GetLastQueriedTime();
			Source.TotalBytes  = PersistentLevelData->TotalMemoryBytes;
			Source.UniqueBytes = PersistentLevelData->UniqueMemoryBytes;
			Source.SharedBytes = PersistentLevelData->SharedMemoryBytes;
			PopulateDrilldownPackages(Source, MemoryEstimator, CachedWorldId);
			AddMemoryProfilerDrilldown(InOutMenuBuilder, MoveTemp(Source));
		}
		else
		{
			FText DiagnosticLabel;
			if (!MemoryEstimator.HasMemorySource())
			{
				DiagnosticLabel = LOCTEXT("PersistentLevelNoMemory", "Allocation data unavailable - recapture with -trace=memalloc,metadata,assetmetadata");
			}
			else if (!bMemoryReady)
			{
				DiagnosticLabel = LOCTEXT("PersistentLevelComputing", "Computing...");
			}
			else
			{
				DiagnosticLabel = LOCTEXT("PersistentLevelNoDependencies", "Dependency data unavailable - recapture with -trace=WorldStreamingDependencies,memalloc,metadata,assetmetadata");
			}
			InOutMenuBuilder.AddMenuEntry(DiagnosticLabel, FText::GetEmpty(), FSlateIcon(), FUIAction(FExecuteAction(), FCanExecuteAction::CreateLambda([](){ return false; })));
		}

		InOutMenuBuilder.EndSection();
		bAddedContent = true;
	}

	for (const UE::Insights::SpatialProfiler::FSpatialPlotRegion& Region : InHitTestResult.Regions)
	{
		WorldStreamingInsightsProvider->ReadStreamingContainer(CachedWorldId, Region.Id, [this, &InOutMenuBuilder, &Region, &bAddedContent](const FStreamingContainerInfo& ContainerInfo)
		{
			InOutMenuBuilder.BeginSection(NAME_None, FText::Format(LOCTEXT("StreamingContainerSection", "Streaming Container: {0}"), Region.Label));
			InOutMenuBuilder.AddMenuEntry(
				LOCTEXT("CopyStreamingContainerNameToClipboard", "Copy Name To Clipboard"),
				LOCTEXT("CopyStreamingContainerNameToClipboardTooltip", "Copy Streaming Container name into the clipboard"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([ClipboardText=FString(ContainerInfo.Name)]() { FPlatformApplicationMisc::ClipboardCopy(*ClipboardText); })),
				NAME_None,
				EUserInterfaceActionType::Button
			);

			if (IsMemoryVisualizationMode(VisualizationMode) && MemoryEstimator.IsReady())
			{
				if (const FContainerMemoryData* ContainerData = MemoryEstimator.GetContainerMemoryData(ContainerInfo.ContainerId))
				{
					const uint64 ContainerId = ContainerInfo.ContainerId;

					FMemoryDrilldownSource Source;
					Source.Label = FText::FromString(ContainerInfo.Name);
					Source.QueryTime   = MemoryEstimator.GetLastQueriedTime();
					Source.TotalBytes  = ContainerData->TotalMemoryBytes;
					Source.UniqueBytes = ContainerData->UniqueMemoryBytes;
					Source.SharedBytes = ContainerData->SharedMemoryBytes;
					PopulateDrilldownPackages(Source, MemoryEstimator, ContainerId);
					AddMemoryProfilerDrilldown(InOutMenuBuilder, MoveTemp(Source));
				}
			}

			InOutMenuBuilder.EndSection();

			bAddedContent = true;
		});
	}

	return bAddedContent;
}

void FWorldStreamingSpatialPlotViewExtender::Tick(const UE::Insights::SpatialProfiler::FSpatialPlotViewExtenderTickParams& InTickParams)
{
	CachedCurrentTraceTime = InTickParams.CurrentTraceTime;

	if (WorldStreamingInsightsProvider && AnalysisSession)
	{
		// Single read scope ensures FindWorldAtTime and RefreshVisibilityCacheIfNeeded
		// see the same provider snapshot.
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);

		uint64 NewWorldId = FindWorldAtTime(CachedCurrentTraceTime);
		if (NewWorldId != CachedWorldId)
		{
			// Clear filter overrides only when switching between two different valid worlds.
			// Don't clear when transitioning to/from UINT64_MAX (no world) - this happens
			// during loading or when scrubbing past world boundaries.
			if (NewWorldId != UINT64_MAX)
			{
				if (LastValidWorldId != UINT64_MAX && NewWorldId != LastValidWorldId)
				{
					ContainerVisibilityOverrides.Empty();
					TagVisibilityOverrides.Empty();
					UntaggedVisibilityOverrides.Empty();
					NewTagDefaultPerGroup.Empty();
					FilterChangeSerial++;
				}
				LastValidWorldId = NewWorldId;
			}
			CachedWorldId = NewWorldId;
		}

		RefreshVisibilityCacheIfNeeded();

		if (IsMemoryVisualizationMode(VisualizationMode))
		{
			const bool bWasReady = MemoryEstimator.IsReady();
			MemoryEstimator.Tick(CachedCurrentTraceTime, CachedWorldId);
			if (!bWasReady && MemoryEstimator.IsReady())
			{
				FilterChangeSerial++;
			}
		}
	}
}

uint32 FWorldStreamingSpatialPlotViewExtender::GetChangeSerial() const
{
	uint32 ChangeSerial = FilterChangeSerial;
	if (WorldStreamingInsightsProvider && AnalysisSession)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);

		ChangeSerial += WorldStreamingInsightsProvider->GetChangeSerial();
	}

	return ChangeSerial;
}

bool FWorldStreamingSpatialPlotViewExtender::HasDataForSession(const TraceServices::IAnalysisSession& InAnalysisSession) const
{
	TraceServices::FAnalysisSessionReadScope SessionReadScope(InAnalysisSession);
	if (const IWorldStreamingInsightsProvider* Provider = InAnalysisSession.ReadProvider<IWorldStreamingInsightsProvider>(GetWorldStreamingInsightsProviderName()))
	{
		return Provider->GetStreamingWorldCount() > 0;
	}
	return false;
}

EStreamingVisualizationMode FWorldStreamingSpatialPlotViewExtender::GetVisualizationMode() const
{
	return VisualizationMode;
}

void FWorldStreamingSpatialPlotViewExtender::SetVisualizationMode(EStreamingVisualizationMode InMode)
{
	VisualizationMode = InMode;
	FilterChangeSerial++;
}

EStreamingPalette FWorldStreamingSpatialPlotViewExtender::GetPalette() const
{
	return Palette;
}

void FWorldStreamingSpatialPlotViewExtender::SetPalette(EStreamingPalette InPalette)
{
	Palette = InPalette;
	FilterChangeSerial++;
}

bool FWorldStreamingSpatialPlotViewExtender::IsFixedScaleEnabled() const
{
	return bFixedScale;
}

void FWorldStreamingSpatialPlotViewExtender::SetFixedScaleEnabled(bool bInEnabled)
{
	bFixedScale = bInEnabled;
	FilterChangeSerial++;
}

int32 FWorldStreamingSpatialPlotViewExtender::GetFixedScaleMax() const
{
	return FixedScaleMaxMiB;
}

void FWorldStreamingSpatialPlotViewExtender::SetFixedScaleMax(int32 InMaxMiB)
{
	FixedScaleMaxMiB = FMath::Max(1, InMaxMiB);
	FilterChangeSerial++;
}

int32 FWorldStreamingSpatialPlotViewExtender::GetAutoScaleMax(EStreamingVisualizationMode InMode) const
{
	int64 MaxBytes = 0;
	switch (InMode)
	{
	case EStreamingVisualizationMode::MemoryTotal:
		MaxBytes = MemoryEstimator.GetMaxContainerMemory();
		break;
	case EStreamingVisualizationMode::MemoryUnique:
		MaxBytes = MemoryEstimator.GetMaxContainerUniqueMemory();
		break;
	case EStreamingVisualizationMode::MemoryShared:
		MaxBytes = MemoryEstimator.GetMaxContainerSharedMemory();
		break;
	default:
		MaxBytes = MemoryEstimator.GetMaxContainerMemory();
		break;
	}
	return FMath::CeilToInt32(static_cast<double>(MaxBytes) / (1024.0 * 1024.0));
}

bool FWorldStreamingSpatialPlotViewExtender::HasMemorySource() const
{
	return MemoryEstimator.HasMemorySource();
}

bool FWorldStreamingSpatialPlotViewExtender::IsMemorySourceReady() const
{
	return MemoryEstimator.IsMemorySourceReady();
}

bool FWorldStreamingSpatialPlotViewExtender::HasDependencyData() const
{
	return MemoryEstimator.HasDependencyData();
}

bool FWorldStreamingSpatialPlotViewExtender::HasPriorityData() const
{
	return bHasPriorityData;
}

void FWorldStreamingSpatialPlotViewExtender::SetContainerVisibility(uint64 InContainerId, bool bInVisible)
{
	ContainerVisibilityOverrides.Add(InContainerId, bInVisible);
	FilterChangeSerial++;
}

void FWorldStreamingSpatialPlotViewExtender::ClearContainerOverride(uint64 InContainerId)
{
	ContainerVisibilityOverrides.Remove(InContainerId);
	FilterChangeSerial++;
}

void FWorldStreamingSpatialPlotViewExtender::ClearAllContainerOverrides()
{
	ContainerVisibilityOverrides.Empty();
	FilterChangeSerial++;
}

void FWorldStreamingSpatialPlotViewExtender::HideAllRootContainers()
{
	if (!WorldStreamingInsightsProvider || !AnalysisSession || CachedWorldId == UINT64_MAX)
	{
		return;
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);

	// Hide only the roots; children inherit hidden via the override resolution walk.
	ContainerVisibilityOverrides.Empty();
	WorldStreamingInsightsProvider->EnumerateRootStreamingContainers(CachedWorldId, [this](const FStreamingContainerInfo& ContainerInfo)
	{
		ContainerVisibilityOverrides.Add(ContainerInfo.ContainerId, false);
	});
	FilterChangeSerial++;
}

bool FWorldStreamingSpatialPlotViewExtender::IsContainerVisible(uint64 InContainerId) const
{
	if (const bool* CachedPtr = ResolvedContainerVisibility.Find(InContainerId))
	{
		return *CachedPtr;
	}
	return true;
}

bool FWorldStreamingSpatialPlotViewExtender::ResolveContainerVisibility(uint64 InContainerId) const
{
	// Walk up the container hierarchy; nearest override wins.
	// If no override found, default to visible.
	uint64 CurrentId = InContainerId;
	while (CurrentId != 0)
	{
		if (const bool* OverridePtr = ContainerVisibilityOverrides.Find(CurrentId))
		{
			return *OverridePtr;
		}

		uint64 ParentId = 0;
		if (WorldStreamingInsightsProvider)
		{
			WorldStreamingInsightsProvider->ReadStreamingContainer(CachedWorldId, CurrentId, [&ParentId](const FStreamingContainerInfo& Info)
			{
				ParentId = Info.ParentId;
			});
		}
		CurrentId = ParentId;
	}
	return true;
}

void FWorldStreamingSpatialPlotViewExtender::SetTagVisibility(uint64 InTagId, bool bInVisible)
{
	TagVisibilityOverrides.Add(InTagId, bInVisible);
	FilterChangeSerial++;
}

void FWorldStreamingSpatialPlotViewExtender::ClearTagOverride(uint64 InTagId)
{
	TagVisibilityOverrides.Remove(InTagId);
	FilterChangeSerial++;
}

void FWorldStreamingSpatialPlotViewExtender::SetUntaggedVisibility(uint64 InGroupId, bool bInVisible)
{
	UntaggedVisibilityOverrides.Add(InGroupId, bInVisible);
	FilterChangeSerial++;
}

void FWorldStreamingSpatialPlotViewExtender::ClearAllTagOverrides()
{
	TagVisibilityOverrides.Empty();
	UntaggedVisibilityOverrides.Empty();
	NewTagDefaultPerGroup.Empty();
	FilterChangeSerial++;
}

bool FWorldStreamingSpatialPlotViewExtender::IsTagVisible(uint64 InTagId) const
{
	if (const bool* CachedPtr = ResolvedTagVisibility.Find(InTagId))
	{
		return *CachedPtr;
	}
	return true;
}

bool FWorldStreamingSpatialPlotViewExtender::ResolveTagVisibility(uint64 InTagId) const
{
	// Walk up the tag hierarchy; nearest override wins.
	// If no override found, use the per-group default.
	uint64 CurrentId = InTagId;
	uint64 GroupId = 0;
	while (CurrentId != 0)
	{
		if (const bool* OverridePtr = TagVisibilityOverrides.Find(CurrentId))
		{
			return *OverridePtr;
		}

		uint64 ParentId = 0;
		if (WorldStreamingInsightsProvider)
		{
			WorldStreamingInsightsProvider->ReadStreamingTag(CurrentId, [&ParentId, &GroupId](const FStreamingTag& Tag)
			{
				ParentId = Tag.ParentId;
				GroupId = Tag.GroupId;
			});
		}
		CurrentId = ParentId;
	}
	return GetNewTagDefaultForGroup(GroupId);
}

bool FWorldStreamingSpatialPlotViewExtender::IsUntaggedVisibleForGroup(uint64 InGroupId) const
{
	if (const bool* OverridePtr = UntaggedVisibilityOverrides.Find(InGroupId))
	{
		return *OverridePtr;
	}
	return true;
}

void FWorldStreamingSpatialPlotViewExtender::PropagateContainerHiddenToAncestors(uint64 InContainerId)
{
	if (!WorldStreamingInsightsProvider || !AnalysisSession || CachedWorldId == UINT64_MAX)
	{
		return;
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);

	uint64 CurrentId = InContainerId;
	while (CurrentId != 0)
	{
		uint64 ParentId = 0;
		WorldStreamingInsightsProvider->ReadStreamingContainer(CachedWorldId, CurrentId, [&ParentId](const FStreamingContainerInfo& Info)
		{
			ParentId = Info.ParentId;
		});

		if (ParentId == 0)
		{
			break;
		}

		// If parent already has a false override, ancestors are already handled
		if (const bool* Override = ContainerVisibilityOverrides.Find(ParentId))
		{
			if (!*Override)
			{
				break;
			}
		}

		// Collect visible siblings BEFORE changing the parent, so we know which ones to protect.
		// This correctly handles the Hide All case: siblings inheriting false from a root won't be in this list.
		TArray<uint64> VisibleSiblingsToProtect;
		WorldStreamingInsightsProvider->EnumerateChildStreamingContainers(CachedWorldId, ParentId, [this, &VisibleSiblingsToProtect](const FStreamingContainerInfo& ChildInfo)
		{
			if (!ContainerVisibilityOverrides.Contains(ChildInfo.ContainerId) && ResolveContainerVisibility(ChildInfo.ContainerId))
			{
				VisibleSiblingsToProtect.Add(ChildInfo.ContainerId);
			}
		});

		ContainerVisibilityOverrides.Add(ParentId, false);

		for (uint64 SiblingId : VisibleSiblingsToProtect)
		{
			ContainerVisibilityOverrides.Add(SiblingId, true);
		}

		CurrentId = ParentId;
	}

	FilterChangeSerial++;
}

void FWorldStreamingSpatialPlotViewExtender::SetNewTagDefaultForGroup(uint64 InGroupId, bool bInVisible)
{
	NewTagDefaultPerGroup.Add(InGroupId, bInVisible);
	FilterChangeSerial++;
}

bool FWorldStreamingSpatialPlotViewExtender::GetNewTagDefaultForGroup(uint64 InGroupId) const
{
	if (const bool* Default = NewTagDefaultPerGroup.Find(InGroupId))
	{
		return *Default;
	}
	return true;
}

uint32 FWorldStreamingSpatialPlotViewExtender::GetProviderChangeSerial() const
{
	if (WorldStreamingInsightsProvider && AnalysisSession)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
		return WorldStreamingInsightsProvider->GetChangeSerial();
	}
	return 0;
}

uint64 FWorldStreamingSpatialPlotViewExtender::GetWorldId() const
{
	return CachedWorldId;
}

void FWorldStreamingSpatialPlotViewExtender::EnumerateRootContainers(TFunctionRef<void(const FStreamingContainerInfo&)> InCallback) const
{
	if (WorldStreamingInsightsProvider && AnalysisSession && CachedWorldId != UINT64_MAX)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
		WorldStreamingInsightsProvider->EnumerateRootStreamingContainers(CachedWorldId, InCallback);
	}
}

void FWorldStreamingSpatialPlotViewExtender::EnumerateChildContainers(uint64 InContainerId, TFunctionRef<void(const FStreamingContainerInfo&)> InCallback) const
{
	if (WorldStreamingInsightsProvider && AnalysisSession && CachedWorldId != UINT64_MAX)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
		WorldStreamingInsightsProvider->EnumerateChildStreamingContainers(CachedWorldId, InContainerId, InCallback);
	}
}

void FWorldStreamingSpatialPlotViewExtender::EnumerateContainers(TFunctionRef<void(const FStreamingContainerInfo&)> InCallback) const
{
	if (WorldStreamingInsightsProvider && AnalysisSession && CachedWorldId != UINT64_MAX)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
		WorldStreamingInsightsProvider->EnumerateStreamingContainers(CachedWorldId, InCallback);
	}
}

void FWorldStreamingSpatialPlotViewExtender::EnumerateTagGroups(TFunctionRef<void(const FStreamingTagGroup&)> InCallback) const
{
	if (WorldStreamingInsightsProvider && AnalysisSession)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
		WorldStreamingInsightsProvider->EnumerateStreamingTagGroups(InCallback);
	}
}

void FWorldStreamingSpatialPlotViewExtender::EnumerateTagsInGroup(uint64 InGroupId, TFunctionRef<void(const FStreamingTag&)> InCallback) const
{
	if (WorldStreamingInsightsProvider && AnalysisSession)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
		WorldStreamingInsightsProvider->EnumerateStreamingTagsInGroup(InGroupId, InCallback);
	}
}

uint32 FWorldStreamingSpatialPlotViewExtender::GetStreamingContainerCount() const
{
	if (WorldStreamingInsightsProvider && AnalysisSession && CachedWorldId != UINT64_MAX)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
		return WorldStreamingInsightsProvider->GetStreamingContainerCount(CachedWorldId);
	}
	return 0;
}

uint64 FWorldStreamingSpatialPlotViewExtender::FindWorldAtTime(double InTime) const
{
	uint64 WorldId = UINT64_MAX;

	if (WorldStreamingInsightsProvider && AnalysisSession)
	{
		bool bFoundWorld = false;

		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);

		// Uses the first world found when multiple are active at the same time.
		WorldStreamingInsightsProvider->EnumerateStreamingWorldsAtTime(InTime, [&WorldId, &bFoundWorld](const FStreamingWorldInfo& WorldInfo)
		{
			if (!bFoundWorld)
			{
				WorldId = WorldInfo.WorldId;
				bFoundWorld = true;
			}
		});
	}

	return WorldId;
}

bool FWorldStreamingSpatialPlotViewExtender::PassesTagFilter(const TArray<uint64>& InContainerTags, const TArray<uint64>& InKnownGroups) const
{
	for (uint64 GroupId : InKnownGroups)
	{
		bool bHasTagInGroup = false;
		bool bHasVisibleTagInGroup = false;

		for (uint64 TagId : InContainerTags)
		{
			WorldStreamingInsightsProvider->ReadStreamingTag(TagId, [this, &GroupId, &bHasTagInGroup, &bHasVisibleTagInGroup](const FStreamingTag& Tag)
			{
				if (Tag.GroupId == GroupId)
				{
					bHasTagInGroup = true;
					if (IsTagVisible(Tag.TagId))
					{
						bHasVisibleTagInGroup = true;
					}
				}
			});
		}

		if (bHasTagInGroup)
		{
			if (!bHasVisibleTagInGroup)
			{
				return false;
			}
		}
		else
		{
			if (!IsUntaggedVisibleForGroup(GroupId))
			{
				return false;
			}
		}
	}

	return true;
}

bool FWorldStreamingSpatialPlotViewExtender::HasControlPanel() const
{
	return true;
}

TSharedPtr<SWidget> FWorldStreamingSpatialPlotViewExtender::CreateControlPanel()
{
	return SNew(SWorldStreamingFilterPanel)
		.Extender(this);
}

void FWorldStreamingSpatialPlotViewExtender::RefreshVisibilityCacheIfNeeded()
{
	check(IsInGameThread());

	if (!WorldStreamingInsightsProvider || !AnalysisSession)
	{
		ResolvedContainerVisibility.Empty();
		ResolvedTagVisibility.Empty();
		return;
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);

	uint32 CurrentProviderSerial = WorldStreamingInsightsProvider->GetChangeSerial();
	if (LastResolvedFilterChangeSerial == FilterChangeSerial && LastResolvedProviderChangeSerial == CurrentProviderSerial)
	{
		return; // cache is fresh
	}

	// When CachedWorldId is UINT64_MAX, keep the old cache so checkboxes show the last known state instead of defaulting to all-checked, and leave the resolved serials stale so a rebuild fires when a valid world returns.
	if (CachedWorldId != UINT64_MAX)
	{
		LastResolvedFilterChangeSerial = FilterChangeSerial;
		LastResolvedProviderChangeSerial = CurrentProviderSerial;

		ResolvedContainerVisibility.Empty();
		WorldStreamingInsightsProvider->EnumerateStreamingContainers(CachedWorldId, [this](const FStreamingContainerInfo& ContainerInfo)
		{
			ResolvedContainerVisibility.Add(ContainerInfo.ContainerId, ResolveContainerVisibility(ContainerInfo.ContainerId));
		});

		ResolvedTagVisibility.Empty();
		WorldStreamingInsightsProvider->EnumerateStreamingTagGroups([this](const FStreamingTagGroup& TagGroup)
		{
			WorldStreamingInsightsProvider->EnumerateStreamingTagsInGroup(TagGroup.GroupId, [this](const FStreamingTag& Tag)
			{
				ResolvedTagVisibility.Add(Tag.TagId, ResolveTagVisibility(Tag.TagId));
			});
		});
	}
}

#undef LOCTEXT_NAMESPACE