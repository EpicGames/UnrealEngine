// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/SpatialProfiler/ISpatialPlotViewExtender.h"
#include "ViewModels/ContainerMemoryEstimator.h"

class IWorldStreamingInsightsProvider;
class FTooltipDrawState;
class SDockTab;
class FSpawnTabArgs;
struct FStreamingContainerInfo;
struct FStreamingTagGroup;
struct FStreamingTag;

enum class EStreamingVisualizationMode : uint8
{
	State,
	Priority,
	MemoryTotal,
	MemoryUnique,
	MemoryShared
};

inline bool IsMemoryVisualizationMode(EStreamingVisualizationMode InMode)
{
	return InMode == EStreamingVisualizationMode::MemoryTotal
		|| InMode == EStreamingVisualizationMode::MemoryUnique
		|| InMode == EStreamingVisualizationMode::MemoryShared;
}

inline bool UsesColorGradient(EStreamingVisualizationMode InMode)
{
	return InMode == EStreamingVisualizationMode::Priority || IsMemoryVisualizationMode(InMode);
}

enum class EStreamingPalette : uint8
{
	Viridis,
	Inferno,
	Grayscale
};

class FWorldStreamingSpatialPlotViewExtender : public UE::Insights::SpatialProfiler::ISpatialPlotViewExtender
{
public:
	//~ Begin ISpatialPlotViewExtender Interface
	virtual void OnBeginSession(const TraceServices::IAnalysisSession& InAnalysisSession) override;
	virtual void OnEndSession() override;

	virtual FName GetLayerName() const override;
	virtual FText GetLayerDisplayName() const override;

	virtual void EnumerateRegions(TFunctionRef<void(const UE::Insights::SpatialProfiler::FSpatialPlotRegion&)> InCallback) override;
	virtual void EnumerateMarkers(TFunctionRef<void(const UE::Insights::SpatialProfiler::FSpatialPlotMarker&)> InCallback) override;

	virtual TOptional<UE::Insights::SpatialProfiler::FSpatialPlotLegend> GetLegend() const override;

	virtual bool AppendTooltip(FTooltipDrawState& InOutTooltip, const UE::Insights::SpatialProfiler::FSpatialPlotHitTestResult& InHitTestResult) const override;
	virtual bool ExtendContextMenu(FMenuBuilder& InOutMenuBuilder, const UE::Insights::SpatialProfiler::FSpatialPlotHitTestResult& InHitTestResult) const override;

	virtual void Tick(const UE::Insights::SpatialProfiler::FSpatialPlotViewExtenderTickParams& InTickParams) override;

	virtual uint32 GetChangeSerial() const override;
	virtual bool HasDataForSession(const TraceServices::IAnalysisSession& InAnalysisSession) const override;
	virtual bool HasControlPanel() const override;
	virtual TSharedPtr<SWidget> CreateControlPanel() override;
	//~ End ISpatialPlotViewExtender Interface

	// Filtering
	void SetContainerVisibility(uint64 InContainerId, bool bInVisible);
	void ClearContainerOverride(uint64 InContainerId);
	void ClearAllContainerOverrides();
	void HideAllRootContainers();
	bool IsContainerVisible(uint64 InContainerId) const;

	void SetTagVisibility(uint64 InTagId, bool bInVisible);
	void ClearTagOverride(uint64 InTagId);
	void SetUntaggedVisibility(uint64 InGroupId, bool bInVisible);
	void ClearAllTagOverrides();
	bool IsTagVisible(uint64 InTagId) const;
	bool IsUntaggedVisibleForGroup(uint64 InGroupId) const;

	void PropagateContainerHiddenToAncestors(uint64 InContainerId);
	void SetNewTagDefaultForGroup(uint64 InGroupId, bool bInVisible);
	bool GetNewTagDefaultForGroup(uint64 InGroupId) const;

	uint32 GetProviderChangeSerial() const;
	uint64 GetWorldId() const;
	void EnumerateRootContainers(TFunctionRef<void(const FStreamingContainerInfo&)> InCallback) const;
	void EnumerateChildContainers(uint64 InContainerId, TFunctionRef<void(const FStreamingContainerInfo&)> InCallback) const;
	void EnumerateContainers(TFunctionRef<void(const FStreamingContainerInfo&)> InCallback) const;
	void EnumerateTagGroups(TFunctionRef<void(const FStreamingTagGroup&)> InCallback) const;
	void EnumerateTagsInGroup(uint64 InGroupId, TFunctionRef<void(const FStreamingTag&)> InCallback) const;
	uint32 GetStreamingContainerCount() const;

	EStreamingVisualizationMode GetVisualizationMode() const;
	void SetVisualizationMode(EStreamingVisualizationMode InMode);

	EStreamingPalette GetPalette() const;
	void SetPalette(EStreamingPalette InPalette);

	bool IsFixedScaleEnabled() const;
	void SetFixedScaleEnabled(bool bInEnabled);
	int32 GetFixedScaleMax() const;
	void SetFixedScaleMax(int32 InMaxMiB);
	int32 GetAutoScaleMax(EStreamingVisualizationMode InMode) const;

	// Diagnostic passthroughs for estimation status.
	bool HasMemorySource() const;
	bool IsMemorySourceReady() const;
	bool HasDependencyData() const;
	bool HasPriorityData() const;

private:
	uint64 FindWorldAtTime(double InTime) const;
	bool PassesTagFilter(const TArray<uint64>& InContainerTags, const TArray<uint64>& InKnownGroups) const;

	const TraceServices::IAnalysisSession* AnalysisSession = nullptr;
	const IWorldStreamingInsightsProvider* WorldStreamingInsightsProvider = nullptr;

	// Override maps: present = explicit show/hide, absent = inherit from parent (containers/tags) or per-group default (NewTagDefaultPerGroup).
	TMap<uint64, bool> ContainerVisibilityOverrides;
	TMap<uint64, bool> TagVisibilityOverrides;
	TMap<uint64, bool> UntaggedVisibilityOverrides;
	TMap<uint64, bool> NewTagDefaultPerGroup;
	uint32 FilterChangeSerial = 0;

	TMap<uint64, bool> ResolvedContainerVisibility;
	TMap<uint64, bool> ResolvedTagVisibility;
	uint32 LastResolvedFilterChangeSerial = UINT32_MAX;
	uint32 LastResolvedProviderChangeSerial = UINT32_MAX;

	void RefreshVisibilityCacheIfNeeded();
	bool ResolveContainerVisibility(uint64 InContainerId) const;
	bool ResolveTagVisibility(uint64 InTagId) const;

	EStreamingVisualizationMode VisualizationMode = EStreamingVisualizationMode::State;
	EStreamingPalette Palette = EStreamingPalette::Viridis;

	uint64 CachedWorldId = UINT64_MAX;
	uint64 LastValidWorldId = UINT64_MAX;
	double CachedCurrentTraceTime = 0.0;

	bool bFixedScale = false;
	bool bHasPriorityData = false;
	int32 FixedScaleMaxMiB = 0;

	FContainerMemoryEstimator MemoryEstimator;
};