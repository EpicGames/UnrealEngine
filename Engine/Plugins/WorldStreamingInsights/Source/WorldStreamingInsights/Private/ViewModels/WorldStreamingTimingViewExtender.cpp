// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/WorldStreamingTimingViewExtender.h"

#include "ViewModels/WorldStreamingTrack.h"

#include "Insights/SpatialProfiler/ISpatialPlotViewExtender.h"

void FWorldStreamingTimingViewExtender::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession)
{
	if (!IsSessionSupported(InSession))
	{
		return;
	}

	WorldStreamingTrack.Reset();
}

void FWorldStreamingTimingViewExtender::OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession)
{
	if (!IsSessionSupported(InSession))
	{
		return;
	}

	AnalysisSession = nullptr;
	WorldStreamingTrack.Reset();
}

void FWorldStreamingTimingViewExtender::Tick(const UE::Insights::Timing::FTimingViewExtenderTickParams& InParams)
{
	if (!IsSessionSupported(InParams.Session))
	{
		return;
	}

	AnalysisSession = InParams.AnalysisSession;

	if (AnalysisSession && !WorldStreamingTrack)
	{
		WorldStreamingTrack = MakeShared<FWorldStreamingTrack>(*this);
		WorldStreamingTrack->SetOrder(FTimingTrackOrder::First);
		WorldStreamingTrack->SetVisibilityFlag(true);
		InParams.Session.AddScrollableTrack(WorldStreamingTrack);
	}
}

const TraceServices::IAnalysisSession* FWorldStreamingTimingViewExtender::GetAnalysisSession() const
{
	return AnalysisSession;
}

bool FWorldStreamingTimingViewExtender::IsSessionSupported(UE::Insights::Timing::ITimingViewSession& InSession) const
{
	return InSession.GetName() == UE::Insights::SpatialProfiler::SpatialProfilerTabId;
}