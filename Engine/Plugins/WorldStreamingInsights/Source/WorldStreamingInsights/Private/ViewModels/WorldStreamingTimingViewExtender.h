// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/ITimingViewExtender.h"

class FWorldStreamingTrack;

class FWorldStreamingTimingViewExtender : public UE::Insights::Timing::ITimingViewExtender
{
public:
	virtual void OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
	virtual void OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
	virtual void Tick(const UE::Insights::Timing::FTimingViewExtenderTickParams& InParams) override;

	const TraceServices::IAnalysisSession* GetAnalysisSession() const;

private:
	bool IsSessionSupported(UE::Insights::Timing::ITimingViewSession& InSession) const;

	const TraceServices::IAnalysisSession* AnalysisSession = nullptr;
	TSharedPtr<FWorldStreamingTrack> WorldStreamingTrack;
};