// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/WorldStreamingTrack.h"

#include "IWorldStreamingInsightsProvider.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "ViewModels/WorldStreamingTimingViewExtender.h"

static const TCHAR* GetNetModeLabel(EStreamingWorldNetMode InNetMode)
{
	switch (InNetMode)
	{
	case EStreamingWorldNetMode::Standalone:
		return TEXT("Standalone");
	case EStreamingWorldNetMode::DedicatedServer:
		return TEXT("Dedicated Server");
	case EStreamingWorldNetMode::ListenServer:
		return TEXT("Listen Server");
	case EStreamingWorldNetMode::Client:
		return TEXT("Client");
	default:
		return TEXT("Unknown");
	}
}

FWorldStreamingTrack::FWorldStreamingTrack(FWorldStreamingTimingViewExtender& InExtender)
	: FTimingEventsTrack("Streaming World")
	, Extender(InExtender)
{
}

void FWorldStreamingTrack::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	const TraceServices::IAnalysisSession* Session = Extender.GetAnalysisSession();
	if (!Session)
	{
		return;
	}

	if (const IWorldStreamingInsightsProvider* WorldStreamingProvider = ReadWorldStreamingInsightsProvider(*Session))
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session);

		// Collect and sort worlds by start time for deterministic depth assignment
		TArray<FStreamingWorldInfo> SortedWorlds;
		WorldStreamingProvider->EnumerateStreamingWorlds([&SortedWorlds](const FStreamingWorldInfo& World)
		{
			SortedWorlds.Add(World);
		});
		SortedWorlds.Sort([](const FStreamingWorldInfo& A, const FStreamingWorldInfo& B)
		{
			return A.StartTime < B.StartTime;
		});

		TArray<double> DepthEndTimes;

		for (const FStreamingWorldInfo& StreamingWorld : SortedWorlds)
		{
			// A source can have multiple entries due to deactivation/reactivation; group all activation periods by source ID.
			TMap<uint64, TArray<FStreamingSourceInfo>> SourcesById;
			TArray<uint64> UniqueSourceIds;
			WorldStreamingProvider->EnumerateStreamingSources(StreamingWorld.WorldId, [&SourcesById, &UniqueSourceIds](const FStreamingSourceInfo& Source)
			{
				if (Source.StartTime != DBL_MAX)
				{
					if (!SourcesById.Contains(Source.SourceId))
					{
						UniqueSourceIds.Add(Source.SourceId);
					}
					SourcesById.FindOrAdd(Source.SourceId).Add(Source);
				}
			});
			const int32 BlockHeight = 1 + UniqueSourceIds.Num();

			// Find lowest base depth where the full block fits without overlap
			int32 BaseDepth = 0;
			while (BaseDepth + BlockHeight <= DepthEndTimes.Num())
			{
				bool bBlockFits = true;
				for (int32 DepthIndex = BaseDepth; DepthIndex < BaseDepth + BlockHeight; DepthIndex++)
				{
					if (DepthEndTimes[DepthIndex] > StreamingWorld.StartTime)
					{
						bBlockFits = false;
						BaseDepth = DepthIndex + 1;
						break;
					}
				}
				if (bBlockFits)
				{
					break;
				}
			}

			const FString WorldLabel = FString::Printf(TEXT("%s (%s)"), StreamingWorld.MapName, GetNetModeLabel(StreamingWorld.NetMode));
			Builder.AddEvent(StreamingWorld.StartTime, StreamingWorld.EndTime, BaseDepth, *WorldLabel);

			int32 SourceDepth = BaseDepth + 1;
			for (uint64 SourceId : UniqueSourceIds)
			{
				const TArray<FStreamingSourceInfo>& Activations = SourcesById[SourceId];
				for (const FStreamingSourceInfo& Activation : Activations)
				{
					double SourceEndTime = FMath::Min(Activation.EndTime, StreamingWorld.EndTime);
					Builder.AddEvent(Activation.StartTime, SourceEndTime, SourceDepth, Activation.Name);
				}
				SourceDepth++;
			}

			while (DepthEndTimes.Num() < BaseDepth + BlockHeight)
			{
				DepthEndTimes.Add(0.0);
			}
			for (int32 DepthIndex = BaseDepth; DepthIndex < BaseDepth + BlockHeight; DepthIndex++)
			{
				DepthEndTimes[DepthIndex] = StreamingWorld.EndTime;
			}
		}
	}
}