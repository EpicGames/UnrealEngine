// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/ViewModels/TimingEventsTrack.h"

class FWorldStreamingTimingViewExtender;

class FWorldStreamingTrack : public FTimingEventsTrack
{
public:
	explicit FWorldStreamingTrack(FWorldStreamingTimingViewExtender& InExtender);
	virtual ~FWorldStreamingTrack() override = default;

	virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;

private:
	FWorldStreamingTimingViewExtender& Extender;
};