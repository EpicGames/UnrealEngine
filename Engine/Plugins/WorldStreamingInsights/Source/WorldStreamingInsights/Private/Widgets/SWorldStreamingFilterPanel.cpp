// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SWorldStreamingFilterPanel.h"

#include "InsightsCore/Common/InsightsCoreStyle.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Text/STextBlock.h"

#include "IWorldStreamingInsightsProvider.h"
#include "ViewModels/WorldStreamingSpatialPlotViewExtender.h"

#define LOCTEXT_NAMESPACE "SWorldStreamingFilterPanel"

////////////////////////////////////////////////////////////////////////////////////////////////////

static FText GetVisualizationModeDisplayText(EStreamingVisualizationMode Mode)
{
	switch (Mode)
	{
	case EStreamingVisualizationMode::State:        return LOCTEXT("ModeState", "State");
	case EStreamingVisualizationMode::Priority:     return LOCTEXT("ModePriority", "Priority");
	case EStreamingVisualizationMode::MemoryTotal:  return LOCTEXT("ModeMemoryTotal", "Memory (Total)");
	case EStreamingVisualizationMode::MemoryUnique: return LOCTEXT("ModeMemoryUnique", "Memory (Unique)");
	case EStreamingVisualizationMode::MemoryShared: return LOCTEXT("ModeMemoryShared", "Memory (Shared)");
	}
	return FText::GetEmpty();
}

void SWorldStreamingFilterPanel::Construct(const FArguments& InArgs)
{
	Extender = InArgs._Extender;

	VisualizationModeOptions.Add(MakeShared<EStreamingVisualizationMode>(EStreamingVisualizationMode::State));
	VisualizationModeOptions.Add(MakeShared<EStreamingVisualizationMode>(EStreamingVisualizationMode::Priority));
	VisualizationModeOptions.Add(MakeShared<EStreamingVisualizationMode>(EStreamingVisualizationMode::MemoryTotal));
	VisualizationModeOptions.Add(MakeShared<EStreamingVisualizationMode>(EStreamingVisualizationMode::MemoryUnique));
	VisualizationModeOptions.Add(MakeShared<EStreamingVisualizationMode>(EStreamingVisualizationMode::MemoryShared));

	ContainerTreeView = SNew(STreeView<TSharedPtr<FContainerTreeItem>>)
		.SelectionMode(ESelectionMode::None)
		.TreeItemsSource(&ContainerTreeRoots)
		.OnGenerateRow(this, &SWorldStreamingFilterPanel::ContainerTree_OnGenerateRow)
		.OnGetChildren(this, &SWorldStreamingFilterPanel::ContainerTree_OnGetChildren);

	TagTreeView = SNew(STreeView<TSharedPtr<FTagTreeItem>>)
		.SelectionMode(ESelectionMode::None)
		.TreeItemsSource(&TagTreeRoots)
		.OnGenerateRow(this, &SWorldStreamingFilterPanel::TagTree_OnGenerateRow)
		.OnGetChildren(this, &SWorldStreamingFilterPanel::TagTree_OnGetChildren);

	ChildSlot
		[
			SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4.0f, 2.0f)
				[
					SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(0.0f, 0.0f, 4.0f, 0.0f)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("Visualization", "Visualization"))
								.Font(FAppStyle::GetFontStyle("NormalFontBold"))
						]

						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						[
							SNew(SComboBox<TSharedPtr<EStreamingVisualizationMode>>)
								.OptionsSource(&VisualizationModeOptions)
								.OnGenerateWidget(this, &SWorldStreamingFilterPanel::GenerateVisualizationModeWidget)
								.OnSelectionChanged(this, &SWorldStreamingFilterPanel::OnVisualizationModeSelected)
								.Content()
								[
									SNew(STextBlock)
										.Text(this, &SWorldStreamingFilterPanel::GetVisualizationModeLabel)
								]
						]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4.0f, 2.0f)
				[
					SNew(SHorizontalBox)
						.IsEnabled_Lambda([this]()
						{
							return UsesColorGradient(Extender->GetVisualizationMode());
						})

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(0.0f, 0.0f, 4.0f, 0.0f)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("Palette", "Palette"))
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SSegmentedControl<EStreamingPalette>)
								.OnValueChanged_Lambda([this](EStreamingPalette InValue)
								{
									Extender->SetPalette(InValue);
								})
								.Value_Lambda([this]()
								{
									return Extender->GetPalette();
								})

								+ SSegmentedControl<EStreamingPalette>::Slot(EStreamingPalette::Viridis)
								.Text(LOCTEXT("PaletteViridis", "Viridis"))
								.ToolTip(LOCTEXT("PaletteViridisToolTip", "Perceptually uniform purple-teal-yellow colormap. Colorblind-friendly."))

								+ SSegmentedControl<EStreamingPalette>::Slot(EStreamingPalette::Inferno)
								.Text(LOCTEXT("PaletteInferno", "Inferno"))
								.ToolTip(LOCTEXT("PaletteInfernoToolTip", "Black-red-orange-yellow colormap. High contrast for outlier detection."))

								+ SSegmentedControl<EStreamingPalette>::Slot(EStreamingPalette::Grayscale)
								.Text(LOCTEXT("PaletteGrayscale", "Grayscale"))
								.ToolTip(LOCTEXT("PaletteGrayscaleToolTip", "Luminance-only ramp. Maximum accessibility."))
						]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(16.0f, 2.0f, 4.0f, 2.0f)
				[
					SNew(SHorizontalBox)
						.Visibility_Lambda([this]()
						{
							return IsMemoryVisualizationMode(Extender->GetVisualizationMode())
								? EVisibility::Visible
								: EVisibility::Collapsed;
						})

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SCheckBox)
								.IsChecked_Lambda([this]()
								{
									return Extender->IsFixedScaleEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
								})
								.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
								{
									const bool bEnabled = (State == ECheckBoxState::Checked);
									Extender->SetFixedScaleEnabled(bEnabled);
									if (bEnabled && Extender->GetFixedScaleMax() == 0)
									{
										Extender->SetFixedScaleMax(FMath::Max(1, Extender->GetAutoScaleMax(Extender->GetVisualizationMode())));
									}
								})
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(4.0f, 0.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("FixedScale", "Fixed Scale:"))
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(4.0f, 0.0f, 0.0f, 0.0f)
						[
							SNew(SNumericEntryBox<int32>)
								.IsEnabled_Lambda([this]()
								{
									return Extender->IsFixedScaleEnabled();
								})
								.Value_Lambda([this]() -> TOptional<int32>
								{
									const int32 Max = Extender->GetFixedScaleMax();
									return Max > 0 ? Max : TOptional<int32>();
								})
								.OnValueCommitted_Lambda([this](int32 InValue, ETextCommit::Type)
								{
									Extender->SetFixedScaleMax(FMath::Max(1, InValue));
								})
								.MinValue(1)
								.AllowSpin(false)
								.MinDesiredValueWidth(40.0f)
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(4.0f, 0.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("FixedScaleUnit", "MiB"))
						]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4.0f, 2.0f)
				[
					SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
						.Visibility(this, &SWorldStreamingFilterPanel::GetMemoryWarningVisibility)
						[
							SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.AutoWidth()
								.Padding(4.0f, 0.0f, 4.0f, 0.0f)
								[
									SNew(SImage)
										.Image(UE::Insights::FInsightsCoreStyle::GetBrush("TreeViewBanner.WarningIcon"))
								]

								+ SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.FillWidth(1.0f)
								.Padding(4.0f, 0.0f, 0.0f, 0.0f)
								[
									SNew(STextBlock)
										.Text(LOCTEXT("MemoryDataWarning", "Allocation data or dependency data not available. Recapture with -trace=WorldStreaming,WorldStreamingDependencies,memalloc,metadata,assetmetadata"))
										.AutoWrapText(true)
								]
						]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(16.0f, 2.0f, 4.0f, 2.0f)
				[
					SNew(STextBlock)
						.Visibility(this, &SWorldStreamingFilterPanel::GetMemoryComputingVisibility)
						.Text(LOCTEXT("ComputingMemory", "Computing memory estimates..."))
						.ColorAndOpacity(FLinearColor(0.5f, 0.5f, 0.5f))
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4.0f, 2.0f)
				[
					SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
						.Visibility(this, &SWorldStreamingFilterPanel::GetPriorityWarningVisibility)
						[
							SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.AutoWidth()
								.Padding(4.0f, 0.0f, 4.0f, 0.0f)
								[
									SNew(SImage)
										.Image(UE::Insights::FInsightsCoreStyle::GetBrush("TreeViewBanner.WarningIcon"))
								]

								+ SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.FillWidth(1.0f)
								.Padding(4.0f, 0.0f, 0.0f, 0.0f)
								[
									SNew(STextBlock)
										.Text(LOCTEXT("PriorityDataWarning", "Priority data not available. Recapture with -trace=WorldStreaming,WorldStreamingPriority"))
										.AutoWrapText(true)
								]
						]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 4.0f, 0.0f, 2.0f)
				[
					SNew(SSeparator)
				]

				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SNew(SSplitter)
						.Orientation(Orient_Vertical)

				+ SSplitter::Slot()
				.Value(0.6f)
				[
					SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(4.0f, 2.0f)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("ContainerHierarchy", "Container Hierarchy"))
								.Font(FAppStyle::GetFontStyle("NormalFontBold"))
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(4.0f, 2.0f)
						[
							SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								[
									SNew(SSearchBox)
										.HintText(LOCTEXT("ContainerSearchHint", "Filter containers..."))
										.OnTextChanged_Lambda([this](const FText& InText)
										{
											const bool bWasFiltering = !ContainerFilterText.IsEmpty();
											ContainerFilterText = InText.ToString();
											const bool bIsFiltering = !ContainerFilterText.IsEmpty();

											if (!bWasFiltering && bIsFiltering)
											{
												PreFilterExpandedIds.Reset();
												TArray<TSharedPtr<FContainerTreeItem>> Stack(ContainerTreeRoots);
												while (Stack.Num() > 0)
												{
													TSharedPtr<FContainerTreeItem> Item = Stack.Pop();
													if (ContainerTreeView->IsItemExpanded(Item))
													{
														PreFilterExpandedIds.Add(Item->ContainerId);
													}
													Stack.Append(Item->Children);
												}
												bHasPreFilterExpansion = true;
											}

											RebuildContainerTree();
										})
								]

								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.Padding(4.0f, 0.0f, 0.0f, 0.0f)
								[
									SNew(STextBlock)
										.Text_Lambda([this]()
										{
											if (ContainerFilterText.IsEmpty())
											{
												return FText::GetEmpty();
											}
											return FText::Format(LOCTEXT("SearchMatchCount", "{0} / {1}"),
												FText::AsNumber(SearchMatchCount), FText::AsNumber(SearchTotalCount));
										})
										.ColorAndOpacity(FSlateColor(FLinearColor(0.5f, 0.5f, 0.5f)))
								]
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(4.0f, 2.0f)
						[
							SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(0.0f, 0.0f, 2.0f, 0.0f)
								[
									SNew(SButton)
										.Text(LOCTEXT("ShowAll", "Show All"))
										.OnClicked_Raw(this, &SWorldStreamingFilterPanel::OnShowAllContainers)
								]

								+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(0.0f, 0.0f, 2.0f, 0.0f)
								[
									SNew(SButton)
										.Text(LOCTEXT("HideAll", "Hide All"))
										.OnClicked_Raw(this, &SWorldStreamingFilterPanel::OnHideAllContainers)
								]

								+ SHorizontalBox::Slot()
								.FillWidth(1.0f)

								+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(0.0f, 0.0f, 2.0f, 0.0f)
								[
									SNew(SButton)
										.Text(LOCTEXT("Expand", "Expand"))
										.OnClicked_Raw(this, &SWorldStreamingFilterPanel::OnExpandAllContainers)
								]

								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SButton)
										.Text(LOCTEXT("Collapse", "Collapse"))
										.OnClicked_Raw(this, &SWorldStreamingFilterPanel::OnCollapseAllContainers)
								]
						]

						+ SVerticalBox::Slot()
						.FillHeight(1.0f)
						[
							ContainerTreeView.ToSharedRef()
						]
				]

				+ SSplitter::Slot()
				.Value(0.4f)
				[
					SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(4.0f, 2.0f)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("Tags", "Tags"))
								.Font(FAppStyle::GetFontStyle("NormalFontBold"))
						]

						+ SVerticalBox::Slot()
						.FillHeight(1.0f)
						[
							TagTreeView.ToSharedRef()
						]
				]
				]
		];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SWorldStreamingFilterPanel::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	uint64 WorldId = Extender->GetWorldId();
	bool bWorldChanged = (WorldId != CachedWorldId);
	if (bWorldChanged)
	{
		CachedWorldId = WorldId;
		// Reset cached counts so the count-based comparisons below mismatch and trigger a rebuild even if the new world happens to have the same counts.
		CachedContainerCount = 0;
		CachedTagGroupCount = 0;
		CachedTagCount = 0;
		bTagTreeNeedsDefaultExpansion = true;
		// Pre-filter expansion IDs are keyed by ContainerId - stale after a world switch.
		bHasPreFilterExpansion = false;
		PreFilterExpandedIds.Reset();
	}

	// Bypass the early-return below when world changed - ProviderChangeSerial only tracks new trace data, not world switches from scrubbing.
	uint32 ProviderChangeSerial = Extender->GetProviderChangeSerial();
	if (ProviderChangeSerial == CachedProviderChangeSerial && !bWorldChanged)
	{
		return;
	}
	CachedProviderChangeSerial = ProviderChangeSerial;

	uint32 ContainerCount = Extender->GetStreamingContainerCount();
	if (ContainerCount != CachedContainerCount)
	{
		CachedContainerCount = ContainerCount;
		RebuildContainerTree();
	}

	uint32 TagGroupCount = 0;
	uint32 TagCount = 0;
	Extender->EnumerateTagGroups([this, &TagGroupCount, &TagCount](const FStreamingTagGroup& TagGroup)
	{
		TagGroupCount++;
		Extender->EnumerateTagsInGroup(TagGroup.GroupId, [&TagCount](const FStreamingTag&)
		{
			TagCount++;
		});
	});

	if (TagGroupCount != CachedTagGroupCount || TagCount != CachedTagCount)
	{
		CachedTagGroupCount = TagGroupCount;
		CachedTagCount = TagCount;
		RebuildTagTree();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Container Tree
////////////////////////////////////////////////////////////////////////////////////////////////////

static TSharedPtr<FContainerTreeItem> BuildContainerTreeItem(FWorldStreamingSpatialPlotViewExtender* InExtender, const FStreamingContainerInfo& InContainerInfo, const FString& InFilterText, int32& InOutTotalNonVirtualCount)
{
	TSharedPtr<FContainerTreeItem> Item = MakeShared<FContainerTreeItem>();
	Item->ContainerId = InContainerInfo.ContainerId;
	Item->Name = InContainerInfo.Name;
	Item->bIsVirtual = !InContainerInfo.Bounds.IsSet();

	// Increment the total before filtering so the count reflects the full data set, not the post-filter view.
	if (!Item->bIsVirtual)
	{
		InOutTotalNonVirtualCount++;
	}

	InExtender->EnumerateChildContainers(InContainerInfo.ContainerId, [InExtender, &Item, &InFilterText, &InOutTotalNonVirtualCount](const FStreamingContainerInfo& ChildInfo)
	{
		TSharedPtr<FContainerTreeItem> Child = BuildContainerTreeItem(InExtender, ChildInfo, InFilterText, InOutTotalNonVirtualCount);
		if (Child.IsValid())
		{
			Item->Children.Add(Child);
		}
	});

	Item->Children.Sort([](const TSharedPtr<FContainerTreeItem>& A, const TSharedPtr<FContainerTreeItem>& B)
	{
		return FCString::Stricmp(*A->Name, *B->Name) < 0;
	});

	Item->RealDescendantCount = 0;
	for (const TSharedPtr<FContainerTreeItem>& Child : Item->Children)
	{
		if (!Child->bIsVirtual)
		{
			Item->RealDescendantCount++;
		}
		Item->RealDescendantCount += Child->RealDescendantCount;
	}

	// When filtering, prune items that don't match and have no matching descendants.
	if (!InFilterText.IsEmpty())
	{
		bool bNameMatches = FCString::Stristr(*Item->Name, *InFilterText) != nullptr;
		if (!bNameMatches && Item->Children.Num() == 0)
		{
			return nullptr;
		}
	}

	return Item;
}

// Count items whose Name contains the search text (excludes virtual and ancestor-only items).
static int32 CountNameMatches(const TArray<TSharedPtr<FContainerTreeItem>>& InRoots, const FString& InFilterText)
{
	if (InFilterText.IsEmpty())
	{
		return 0;
	}

	int32 Count = 0;
	TArray<TSharedPtr<FContainerTreeItem>> Stack(InRoots);
	while (Stack.Num() > 0)
	{
		TSharedPtr<FContainerTreeItem> Item = Stack.Pop();
		if (!Item->bIsVirtual && FCString::Stristr(*Item->Name, *InFilterText) != nullptr)
		{
			Count++;
		}
		Stack.Append(Item->Children);
	}
	return Count;
}

void SWorldStreamingFilterPanel::RebuildContainerTree()
{
	const bool bIsFiltering = !ContainerFilterText.IsEmpty();

	// Capture current expansion state - but only when not filtering.
	// During filtering everything is force-expanded, so capturing would overwrite the real pre-filter expansion state.
	TSet<uint64> ExpandedIds;
	if (!bIsFiltering && !bHasPreFilterExpansion)
	{
		TArray<TSharedPtr<FContainerTreeItem>> Stack(ContainerTreeRoots);
		while (Stack.Num() > 0)
		{
			TSharedPtr<FContainerTreeItem> Item = Stack.Pop();
			if (ContainerTreeView->IsItemExpanded(Item))
			{
				ExpandedIds.Add(Item->ContainerId);
			}
			Stack.Append(Item->Children);
		}
	}

	ContainerTreeRoots.Empty();

	int32 TotalNonVirtualCount = 0;
	const uint64 WorldId = Extender->GetWorldId();
	Extender->EnumerateRootContainers([this, &TotalNonVirtualCount, WorldId](const FStreamingContainerInfo& ContainerInfo)
	{
		// The persistent-level container has ContainerId == WorldId. It's reachable via the spatial-plot context menu; no value listing it here (no children, not filterable).
		if (ContainerInfo.ContainerId == WorldId)
		{
			return;
		}

		TSharedPtr<FContainerTreeItem> Root = BuildContainerTreeItem(Extender, ContainerInfo, ContainerFilterText, TotalNonVirtualCount);
		if (Root.IsValid())
		{
			ContainerTreeRoots.Add(Root);
		}
	});

	ContainerTreeRoots.Sort([](const TSharedPtr<FContainerTreeItem>& A, const TSharedPtr<FContainerTreeItem>& B)
	{
		return FCString::Stricmp(*A->Name, *B->Name) < 0;
	});

	SearchTotalCount = TotalNonVirtualCount;
	SearchMatchCount = CountNameMatches(ContainerTreeRoots, ContainerFilterText);

	TArray<TSharedPtr<FContainerTreeItem>> Stack(ContainerTreeRoots);
	while (Stack.Num() > 0)
	{
		TSharedPtr<FContainerTreeItem> Item = Stack.Pop();
		if (bIsFiltering)
		{
			ContainerTreeView->SetItemExpansion(Item, true);
		}
		else if (bHasPreFilterExpansion)
		{
			ContainerTreeView->SetItemExpansion(Item, PreFilterExpandedIds.Contains(Item->ContainerId));
		}
		else if (ExpandedIds.Contains(Item->ContainerId))
		{
			ContainerTreeView->SetItemExpansion(Item, true);
		}
		Stack.Append(Item->Children);
	}

	if (!bIsFiltering && bHasPreFilterExpansion)
	{
		bHasPreFilterExpansion = false;
	}

	ContainerTreeView->RequestTreeRefresh();
}

TSharedRef<ITableRow> SWorldStreamingFilterPanel::ContainerTree_OnGenerateRow(TSharedPtr<FContainerTreeItem> InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	const bool bIsVirtual = InItem->bIsVirtual;
	const FSlateColor TextColor = bIsVirtual ? FSlateColor(FLinearColor(0.5f, 0.5f, 0.5f)) : FSlateColor::UseForeground();
	const FSlateFontInfo Font = bIsVirtual ? FAppStyle::GetFontStyle("ItalicFont") : FAppStyle::GetFontStyle("NormalFont");

	return SNew(STableRow<TSharedPtr<FContainerTreeItem>>, InOwnerTable)
		[
			SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
						.IsChecked(this, &SWorldStreamingFilterPanel::ContainerTree_GetCheckState, InItem)
						.OnCheckStateChanged(this, &SWorldStreamingFilterPanel::ContainerTree_OnCheckStateChanged, InItem)
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(2.0f, 0.0f)
				[
					SNew(STextBlock)
						.Text(FText::FromString(InItem->Name))
						.ToolTipText(FText::FromString(InItem->Name))
						.Font(Font)
						.ColorAndOpacity(TextColor)
				]

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2.0f, 0.0f)
				[
					SNew(STextBlock)
						.Text_Lambda([Item = InItem]()
							{
								if (Item->bIsVirtual || Item->Children.Num() > 0)
								{
									return FText::Format(LOCTEXT("CountFormat", "({0})"), FText::AsNumber(Item->RealDescendantCount));
								}
								return FText::GetEmpty();
							})
						.ColorAndOpacity(FSlateColor(FLinearColor(0.5f, 0.5f, 0.5f)))
				]
		];
}

void SWorldStreamingFilterPanel::ContainerTree_OnGetChildren(TSharedPtr<FContainerTreeItem> InItem, TArray<TSharedPtr<FContainerTreeItem>>& OutChildren)
{
	OutChildren = InItem->Children;
}

ECheckBoxState SWorldStreamingFilterPanel::ContainerTree_GetCheckState(TSharedPtr<FContainerTreeItem> InItem) const
{
	if (InItem->Children.Num() == 0)
	{
		return Extender->IsContainerVisible(InItem->ContainerId) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	int32 NumChecked = 0;
	int32 NumUnchecked = 0;

	TArray<TSharedPtr<FContainerTreeItem>> Stack;
	Stack.Append(InItem->Children);
	while (Stack.Num() > 0)
	{
		TSharedPtr<FContainerTreeItem> Current = Stack.Pop();
		if (Current->Children.Num() == 0)
		{
			if (Extender->IsContainerVisible(Current->ContainerId))
			{
				NumChecked++;
			}
			else
			{
				NumUnchecked++;
			}

			if (NumChecked > 0 && NumUnchecked > 0)
			{
				return ECheckBoxState::Undetermined;
			}
		}
		else
		{
			Stack.Append(Current->Children);
		}
	}

	if (NumChecked == 0 && NumUnchecked == 0)
	{
		return ECheckBoxState::Checked; // No leaves found - default to visible.
	}
	if (NumChecked == 0)
	{
		return ECheckBoxState::Unchecked;
	}
	if (NumUnchecked == 0)
	{
		return ECheckBoxState::Checked;
	}
	return ECheckBoxState::Undetermined;
}

void SWorldStreamingFilterPanel::ContainerTree_OnCheckStateChanged(ECheckBoxState InNewState, TSharedPtr<FContainerTreeItem> InItem)
{
	ECheckBoxState CurrentState = ContainerTree_GetCheckState(InItem);
	bool bNewVisible = (CurrentState == ECheckBoxState::Unchecked);
	ContainerTree_SetVisibilityRecursive(InItem, bNewVisible);

	if (!bNewVisible)
	{
		Extender->PropagateContainerHiddenToAncestors(InItem->ContainerId);
	}
}

void SWorldStreamingFilterPanel::ContainerTree_SetVisibilityRecursive(TSharedPtr<FContainerTreeItem> InItem, bool bInVisible)
{
	// Set override on this node; clear overrides on all descendants so they inherit from this node.
	// When a search filter is active, only filtered (visible) descendants are cleared - non-matching descendants retain their previous overrides until the filter is cleared.
	Extender->SetContainerVisibility(InItem->ContainerId, bInVisible);

	for (const TSharedPtr<FContainerTreeItem>& Child : InItem->Children)
	{
		ContainerTree_ClearOverridesRecursive(Child);
	}
}

void SWorldStreamingFilterPanel::ContainerTree_ClearOverridesRecursive(TSharedPtr<FContainerTreeItem> InItem)
{
	Extender->ClearContainerOverride(InItem->ContainerId);
	for (const TSharedPtr<FContainerTreeItem>& Child : InItem->Children)
	{
		ContainerTree_ClearOverridesRecursive(Child);
	}
}

FReply SWorldStreamingFilterPanel::OnShowAllContainers()
{
	Extender->ClearAllContainerOverrides();
	return FReply::Handled();
}

FReply SWorldStreamingFilterPanel::OnHideAllContainers()
{
	Extender->HideAllRootContainers();
	return FReply::Handled();
}

FReply SWorldStreamingFilterPanel::OnExpandAllContainers()
{
	for (const TSharedPtr<FContainerTreeItem>& Root : ContainerTreeRoots)
	{
		ExpandContainerTreeRecursive(Root, true);
	}
	return FReply::Handled();
}

FReply SWorldStreamingFilterPanel::OnCollapseAllContainers()
{
	for (const TSharedPtr<FContainerTreeItem>& Root : ContainerTreeRoots)
	{
		ExpandContainerTreeRecursive(Root, false);
	}
	return FReply::Handled();
}

void SWorldStreamingFilterPanel::ExpandContainerTreeRecursive(TSharedPtr<FContainerTreeItem> InItem, bool bInExpand)
{
	ContainerTreeView->SetItemExpansion(InItem, bInExpand);
	for (const TSharedPtr<FContainerTreeItem>& Child : InItem->Children)
	{
		ExpandContainerTreeRecursive(Child, bInExpand);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Tag Tree
////////////////////////////////////////////////////////////////////////////////////////////////////

static void SortTagTreeItemChildren(TArray<TSharedPtr<FTagTreeItem>>& InOutChildren)
{
	InOutChildren.Sort([](const TSharedPtr<FTagTreeItem>& A, const TSharedPtr<FTagTreeItem>& B)
	{
		return FCString::Stricmp(*A->Name, *B->Name) < 0;
	});

	for (const TSharedPtr<FTagTreeItem>& Child : InOutChildren)
	{
		if (Child->Children.Num() > 0)
		{
			SortTagTreeItemChildren(Child->Children);
		}
	}
}

void SWorldStreamingFilterPanel::RebuildTagTree()
{
	TSet<uint64> ExpandedIds;
	TArray<TSharedPtr<FTagTreeItem>> Stack(TagTreeRoots);
	while (Stack.Num() > 0)
	{
		TSharedPtr<FTagTreeItem> Item = Stack.Pop();
		if (TagTreeView->IsItemExpanded(Item))
		{
			ExpandedIds.Add(Item->Id);
		}
		Stack.Append(Item->Children);
	}

	TagTreeRoots.Empty();

	TMap<uint64, int32> TagContainerCounts;

	Extender->EnumerateContainers([this, &TagContainerCounts](const FStreamingContainerInfo& ContainerInfo)
	{
		for (uint64 TagId : ContainerInfo.Tags)
		{
			TagContainerCounts.FindOrAdd(TagId, 0)++;
		}
	});

	Extender->EnumerateTagGroups([this, &TagContainerCounts](const FStreamingTagGroup& TagGroup)
	{
		TSharedPtr<FTagTreeItem> GroupItem = MakeShared<FTagTreeItem>();
		GroupItem->Kind = FTagTreeItem::EKind::Group;
		GroupItem->Id = TagGroup.GroupId;
		GroupItem->GroupId = TagGroup.GroupId;
		GroupItem->Name = TagGroup.Name;
		GroupItem->ContainerCount = 0;

		TMap<uint64, TArray<TSharedPtr<FTagTreeItem>>> TagsByParent;
		TMap<uint64, TSharedPtr<FTagTreeItem>> TagItemsById;

		Extender->EnumerateTagsInGroup(TagGroup.GroupId, [&TagGroup, &TagContainerCounts, &TagItemsById, &TagsByParent](const FStreamingTag& StreamingTag)
		{
			TSharedPtr<FTagTreeItem> TagItem = MakeShared<FTagTreeItem>();
			TagItem->Kind = FTagTreeItem::EKind::Tag;
			TagItem->Id = StreamingTag.TagId;
			TagItem->GroupId = TagGroup.GroupId;
			TagItem->Name = StreamingTag.Name;
			TagItem->ContainerCount = TagContainerCounts.FindRef(StreamingTag.TagId);

			TagItemsById.Add(StreamingTag.TagId, TagItem);
			TagsByParent.FindOrAdd(StreamingTag.ParentId).Add(TagItem);
		});

		int32 UntaggedCount = 0;
		Extender->EnumerateContainers([&TagItemsById, &UntaggedCount](const FStreamingContainerInfo& ContainerInfo)
		{
			bool bHasTagInGroup = false;
			for (uint64 TagId : ContainerInfo.Tags)
			{
				if (TagItemsById.Contains(TagId))
				{
					bHasTagInGroup = true;
					break;
				}
			}
			if (!bHasTagInGroup)
			{
				UntaggedCount++;
			}
		});

		for (auto& Pair : TagsByParent)
		{
			if (Pair.Key == 0)
			{
				for (const TSharedPtr<FTagTreeItem>& TagItem : Pair.Value)
				{
					GroupItem->Children.Add(TagItem);
				}
			}
			else
			{
				TSharedPtr<FTagTreeItem>* ParentPtr = TagItemsById.Find(Pair.Key);
				if (ParentPtr)
				{
					for (const TSharedPtr<FTagTreeItem>& TagItem : Pair.Value)
					{
						(*ParentPtr)->Children.Add(TagItem);
					}
				}
			}
		}

		// Sort before inserting the Untagged pseudo-item so it stays pinned at index 0.
		SortTagTreeItemChildren(GroupItem->Children);

		TSharedPtr<FTagTreeItem> UntaggedItem = MakeShared<FTagTreeItem>();
		UntaggedItem->Kind = FTagTreeItem::EKind::Untagged;
		UntaggedItem->Id = TagGroup.GroupId;
		UntaggedItem->GroupId = TagGroup.GroupId;
		UntaggedItem->Name = TagGroup.UntaggedLabel;
		UntaggedItem->ContainerCount = UntaggedCount;

		GroupItem->Children.Insert(UntaggedItem, 0);

		for (const TSharedPtr<FTagTreeItem>& Child : GroupItem->Children)
		{
			GroupItem->ContainerCount += Child->ContainerCount;
		}

		TagTreeRoots.Add(GroupItem);
	});

	Stack = TagTreeRoots;
	while (Stack.Num() > 0)
	{
		TSharedPtr<FTagTreeItem> Item = Stack.Pop();
		const bool bShouldExpand = bTagTreeNeedsDefaultExpansion ? (Item->Kind == FTagTreeItem::EKind::Group) : ExpandedIds.Contains(Item->Id);
		if (bShouldExpand)
		{
			TagTreeView->SetItemExpansion(Item, true);
		}
		Stack.Append(Item->Children);
	}

	if (TagTreeRoots.Num() > 0)
	{
		bTagTreeNeedsDefaultExpansion = false;
	}

	TagTreeView->RequestTreeRefresh();
}

TSharedRef<ITableRow> SWorldStreamingFilterPanel::TagTree_OnGenerateRow(TSharedPtr<FTagTreeItem> InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	const bool bIsDimmed = (InItem->Kind == FTagTreeItem::EKind::Untagged);
	const FSlateColor TextColor = bIsDimmed ? FSlateColor(FLinearColor(0.5f, 0.5f, 0.5f)) : FSlateColor::UseForeground();
	const FSlateFontInfo Font = bIsDimmed ? FAppStyle::GetFontStyle("ItalicFont") : (InItem->Kind == FTagTreeItem::EKind::Group ? FAppStyle::GetFontStyle("NormalFontBold") : FAppStyle::GetFontStyle("NormalFont"));

	return SNew(STableRow<TSharedPtr<FTagTreeItem>>, InOwnerTable)
		[
			SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
						.IsChecked(this, &SWorldStreamingFilterPanel::TagTree_GetCheckState, InItem)
						.OnCheckStateChanged(this, &SWorldStreamingFilterPanel::TagTree_OnCheckStateChanged, InItem)
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(2.0f, 0.0f)
				[
					SNew(STextBlock)
						.Text(FText::FromString(InItem->Name))
						.ToolTipText(FText::FromString(InItem->Name))
						.Font(Font)
						.ColorAndOpacity(TextColor)
				]

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2.0f, 0.0f)
				[
					SNew(STextBlock)
						.Text(FText::Format(LOCTEXT("TagCountFormat", "({0})"), FText::AsNumber(InItem->ContainerCount)))
						.ColorAndOpacity(FSlateColor(FLinearColor(0.5f, 0.5f, 0.5f)))
				]
		];
}

void SWorldStreamingFilterPanel::TagTree_OnGetChildren(TSharedPtr<FTagTreeItem> InItem, TArray<TSharedPtr<FTagTreeItem>>& OutChildren)
{
	OutChildren = InItem->Children;
}

ECheckBoxState SWorldStreamingFilterPanel::TagTree_GetCheckState(TSharedPtr<FTagTreeItem> InItem) const
{
	if (InItem->Kind == FTagTreeItem::EKind::Untagged)
	{
		return Extender->IsUntaggedVisibleForGroup(InItem->GroupId) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	if (InItem->Kind == FTagTreeItem::EKind::Tag && InItem->Children.Num() == 0)
	{
		return Extender->IsTagVisible(InItem->Id) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	// Parent node (Group or Tag with children): compute tri-state from children
	int32 NumChecked = 0;
	int32 NumUnchecked = 0;

	TArray<TSharedPtr<FTagTreeItem>> Stack;
	Stack.Append(InItem->Children);
	while (Stack.Num() > 0)
	{
		TSharedPtr<FTagTreeItem> Current = Stack.Pop();
		if (Current->Children.Num() == 0 || Current->Kind == FTagTreeItem::EKind::Untagged)
		{
			ECheckBoxState State = TagTree_GetCheckState(Current);
			if (State == ECheckBoxState::Checked)
			{
				NumChecked++;
			}
			else
			{
				NumUnchecked++;
			}

			if (NumChecked > 0 && NumUnchecked > 0)
			{
				return ECheckBoxState::Undetermined;
			}
		}
		else
		{
			Stack.Append(Current->Children);
		}
	}

	if (NumChecked == 0)
	{
		return ECheckBoxState::Unchecked;
	}
	if (NumUnchecked == 0)
	{
		return ECheckBoxState::Checked;
	}
	return ECheckBoxState::Undetermined;
}

void SWorldStreamingFilterPanel::TagTree_OnCheckStateChanged(ECheckBoxState InNewState, TSharedPtr<FTagTreeItem> InItem)
{
	ECheckBoxState CurrentState = TagTree_GetCheckState(InItem);
	bool bNewVisible = (CurrentState == ECheckBoxState::Unchecked);
	TagTree_SetVisibilityRecursive(InItem, bNewVisible);

	if (InItem->Kind != FTagTreeItem::EKind::Untagged)
	{
		Extender->SetNewTagDefaultForGroup(InItem->GroupId, bNewVisible);
	}
}

void SWorldStreamingFilterPanel::TagTree_SetVisibilityRecursive(TSharedPtr<FTagTreeItem> InItem, bool bInVisible)
{
	if (InItem->Kind == FTagTreeItem::EKind::Untagged)
	{
		Extender->SetUntaggedVisibility(InItem->GroupId, bInVisible);
	}
	else if (InItem->Kind == FTagTreeItem::EKind::Tag)
	{
		Extender->SetTagVisibility(InItem->Id, bInVisible);
		for (const TSharedPtr<FTagTreeItem>& Child : InItem->Children)
		{
			TagTree_ClearOverridesRecursive(Child);
		}
	}
	else if (InItem->Kind == FTagTreeItem::EKind::Group)
	{
		for (const TSharedPtr<FTagTreeItem>& Child : InItem->Children)
		{
			TagTree_SetVisibilityRecursive(Child, bInVisible);
		}
	}
}

void SWorldStreamingFilterPanel::TagTree_ClearOverridesRecursive(TSharedPtr<FTagTreeItem> InItem)
{
	if (InItem->Kind == FTagTreeItem::EKind::Tag)
	{
		Extender->ClearTagOverride(InItem->Id);
	}
	for (const TSharedPtr<FTagTreeItem>& Child : InItem->Children)
	{
		TagTree_ClearOverridesRecursive(Child);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Visualization Mode Combo Box
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SWorldStreamingFilterPanel::GenerateVisualizationModeWidget(TSharedPtr<EStreamingVisualizationMode> InMode)
{
	return SNew(STextBlock)
		.Text(GetVisualizationModeDisplayText(*InMode));
}

void SWorldStreamingFilterPanel::OnVisualizationModeSelected(TSharedPtr<EStreamingVisualizationMode> InMode, ESelectInfo::Type InSelectInfo)
{
	if (InMode.IsValid())
	{
		Extender->SetVisualizationMode(*InMode);
	}
}

FText SWorldStreamingFilterPanel::GetVisualizationModeLabel() const
{
	return GetVisualizationModeDisplayText(Extender->GetVisualizationMode());
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Memory Estimation Status
////////////////////////////////////////////////////////////////////////////////////////////////////

EVisibility SWorldStreamingFilterPanel::GetMemoryWarningVisibility() const
{
	if (Extender->GetVisualizationMode() < EStreamingVisualizationMode::MemoryTotal)
	{
		return EVisibility::Collapsed;
	}

	if (!Extender->HasMemorySource())
	{
		return EVisibility::Visible;
	}

	if (!Extender->IsMemorySourceReady())
	{
		return EVisibility::Collapsed;
	}

	if (!Extender->HasDependencyData())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

EVisibility SWorldStreamingFilterPanel::GetMemoryComputingVisibility() const
{
	if (Extender->GetVisualizationMode() < EStreamingVisualizationMode::MemoryTotal)
	{
		return EVisibility::Collapsed;
	}

	if (!Extender->HasMemorySource())
	{
		return EVisibility::Collapsed;
	}

	if (!Extender->IsMemorySourceReady())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

EVisibility SWorldStreamingFilterPanel::GetPriorityWarningVisibility() const
{
	if (Extender->GetVisualizationMode() != EStreamingVisualizationMode::Priority)
	{
		return EVisibility::Collapsed;
	}

	if (!Extender->HasPriorityData())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE