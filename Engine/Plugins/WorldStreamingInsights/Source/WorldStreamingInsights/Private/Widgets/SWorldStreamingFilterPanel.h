// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

class FWorldStreamingSpatialPlotViewExtender;
enum class EStreamingVisualizationMode : uint8;

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FContainerTreeItem
{
	uint64 ContainerId = 0;
	FString Name;
	bool bIsVirtual = false;
	int32 RealDescendantCount = 0;
	TArray<TSharedPtr<FContainerTreeItem>> Children;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FTagTreeItem
{
	enum class EKind : uint8
	{
		Group,
		Untagged,
		Tag
	};

	EKind Kind = EKind::Tag;
	uint64 Id = 0;
	uint64 GroupId = 0;
	FString Name;
	int32 ContainerCount = 0;
	TArray<TSharedPtr<FTagTreeItem>> Children;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class SWorldStreamingFilterPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SWorldStreamingFilterPanel) {}
		SLATE_ARGUMENT(FWorldStreamingSpatialPlotViewExtender*, Extender)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	// Container tree
	void RebuildContainerTree();
	TSharedRef<ITableRow> ContainerTree_OnGenerateRow(TSharedPtr<FContainerTreeItem> InItem, const TSharedRef<STableViewBase>& InOwnerTable);
	void ContainerTree_OnGetChildren(TSharedPtr<FContainerTreeItem> InItem, TArray<TSharedPtr<FContainerTreeItem>>& OutChildren);
	ECheckBoxState ContainerTree_GetCheckState(TSharedPtr<FContainerTreeItem> InItem) const;
	void ContainerTree_OnCheckStateChanged(ECheckBoxState InNewState, TSharedPtr<FContainerTreeItem> InItem);
	void ContainerTree_SetVisibilityRecursive(TSharedPtr<FContainerTreeItem> InItem, bool bInVisible);
	void ContainerTree_ClearOverridesRecursive(TSharedPtr<FContainerTreeItem> InItem);

	FReply OnShowAllContainers();
	FReply OnHideAllContainers();
	FReply OnExpandAllContainers();
	FReply OnCollapseAllContainers();
	void ExpandContainerTreeRecursive(TSharedPtr<FContainerTreeItem> InItem, bool bInExpand);

	// Tag tree
	void RebuildTagTree();
	TSharedRef<ITableRow> TagTree_OnGenerateRow(TSharedPtr<FTagTreeItem> InItem, const TSharedRef<STableViewBase>& InOwnerTable);
	void TagTree_OnGetChildren(TSharedPtr<FTagTreeItem> InItem, TArray<TSharedPtr<FTagTreeItem>>& OutChildren);
	ECheckBoxState TagTree_GetCheckState(TSharedPtr<FTagTreeItem> InItem) const;
	void TagTree_OnCheckStateChanged(ECheckBoxState InNewState, TSharedPtr<FTagTreeItem> InItem);
	void TagTree_SetVisibilityRecursive(TSharedPtr<FTagTreeItem> InItem, bool bInVisible);
	void TagTree_ClearOverridesRecursive(TSharedPtr<FTagTreeItem> InItem);

	// Visualization mode combo box
	TSharedRef<SWidget> GenerateVisualizationModeWidget(TSharedPtr<EStreamingVisualizationMode> InMode);
	void OnVisualizationModeSelected(TSharedPtr<EStreamingVisualizationMode> InMode, ESelectInfo::Type InSelectInfo);
	FText GetVisualizationModeLabel() const;

	TArray<TSharedPtr<EStreamingVisualizationMode>> VisualizationModeOptions;

	FWorldStreamingSpatialPlotViewExtender* Extender = nullptr;
	uint32 CachedProviderChangeSerial = 0;
	uint32 CachedContainerCount = 0;
	uint32 CachedTagGroupCount = 0;
	uint32 CachedTagCount = 0;

	FString ContainerFilterText;
	int32 SearchMatchCount = 0;
	int32 SearchTotalCount = 0;

	TSet<uint64> PreFilterExpandedIds;
	bool bHasPreFilterExpansion = false;

	TArray<TSharedPtr<FContainerTreeItem>> ContainerTreeRoots;
	TSharedPtr<STreeView<TSharedPtr<FContainerTreeItem>>> ContainerTreeView;

	bool bTagTreeNeedsDefaultExpansion = true;
	TArray<TSharedPtr<FTagTreeItem>> TagTreeRoots;
	TSharedPtr<STreeView<TSharedPtr<FTagTreeItem>>> TagTreeView;

	uint64 CachedWorldId = UINT64_MAX;

	// Memory estimation status - evaluated every frame by Slate via method binding.
	EVisibility GetMemoryWarningVisibility() const;
	EVisibility GetMemoryComputingVisibility() const;
	EVisibility GetPriorityWarningVisibility() const;
};