// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldStreamingInsightsModule.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"
#include "WorldStreamingInsightsLog.h"

DEFINE_LOG_CATEGORY(LogWorldStreamingInsights);

FWorldStreamingInsightsModule& FWorldStreamingInsightsModule::Get()
{
	return FModuleManager::LoadModuleChecked<FWorldStreamingInsightsModule>("WorldStreamingInsights");
}

void FWorldStreamingInsightsModule::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &TraceModule);
	IModularFeatures::Get().RegisterModularFeature(UE::Insights::SpatialProfiler::SpatialPlotViewExtenderFeatureName, &SpatialPlotViewExtender);
	IModularFeatures::Get().RegisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &TimingViewExtender);
}

void FWorldStreamingInsightsModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &TimingViewExtender);
	IModularFeatures::Get().UnregisterModularFeature(UE::Insights::SpatialProfiler::SpatialPlotViewExtenderFeatureName, &SpatialPlotViewExtender);
	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &TraceModule);
}

IMPLEMENT_MODULE(FWorldStreamingInsightsModule, WorldStreamingInsights);