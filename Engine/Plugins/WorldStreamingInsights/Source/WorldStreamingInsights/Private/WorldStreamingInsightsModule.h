// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/WorldStreamingSpatialPlotViewExtender.h"
#include "ViewModels/WorldStreamingTimingViewExtender.h"
#include "Modules/ModuleInterface.h"
#include "WorldStreamingInsightsTraceModule.h"

class FWorldStreamingInsightsModule : public IModuleInterface
{
public:
	static FWorldStreamingInsightsModule& Get();

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	FWorldStreamingInsightsTraceModule TraceModule;
	FWorldStreamingSpatialPlotViewExtender SpatialPlotViewExtender;
	FWorldStreamingTimingViewExtender TimingViewExtender;
};