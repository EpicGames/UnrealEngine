// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldStreamingInsightsTraceModule.h"
#include "Model/WorldStreamingInsightsProvider.h"
#include "Analyzers/WorldStreamingInsightsAnalyzer.h"
#include "TraceServices/Model/AnalysisSession.h"

FName FWorldStreamingInsightsTraceModule::ModuleName(TEXT("TraceModule_WorldStreaming"));

void FWorldStreamingInsightsTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = ModuleName;
	OutModuleInfo.DisplayName = TEXT("WorldStreaming");
}

void FWorldStreamingInsightsTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
{
	TSharedPtr<FWorldStreamingInsightsProvider> Provider = MakeShared<FWorldStreamingInsightsProvider>(InSession);
	InSession.AddProvider(GetWorldStreamingInsightsProviderName(), Provider);
	InSession.AddAnalyzer(new FWorldStreamingInsightsAnalyzer(InSession, *Provider));
}

void FWorldStreamingInsightsTraceModule::GetLoggers(TArray<const TCHAR*>& OutLoggers)
{
	OutLoggers.Add(TEXT("WorldStreaming"));
}

void FWorldStreamingInsightsTraceModule::GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
{
}