// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Box.h"
#include "Math/Vector.h"
#include "Misc/Optional.h"
#include "TraceServices/Model/AnalysisSession.h"

#define UE_API WORLDSTREAMINGINSIGHTS_API

// EXPERIMENTAL: This API mirrors the experimental World Streaming trace event format and is subject to change without deprecation.
// Code that depends on these names or shapes may need updates between engine versions.

enum class UE_EXPERIMENTAL(5.8, "World Streaming Insights API is experimental and subject to change.") EStreamingContainerState : uint8
{
	Unloaded,
	Loading,
	Loaded,
	Activating,
	Active,
	Deactivating
};

enum class UE_EXPERIMENTAL(5.8, "World Streaming Insights API is experimental and subject to change.") EStreamingWorldNetMode : uint8
{
	Standalone,
	DedicatedServer,
	ListenServer,
	Client
};

struct UE_EXPERIMENTAL(5.8, "World Streaming Insights API is experimental and subject to change.") FStreamingWorldInfo
{
	uint64 WorldId = 0;
	const TCHAR* MapName = nullptr;
	EStreamingWorldNetMode NetMode = EStreamingWorldNetMode::Standalone;
	double StartTime = 0.0;
	double EndTime = 0.0;
};

struct UE_EXPERIMENTAL(5.8, "World Streaming Insights API is experimental and subject to change.") FStreamingContainerInfo
{
	uint64 ContainerId = 0;
	uint64 ParentId = 0;
	const TCHAR* Name = nullptr;
	const TCHAR* PackageName = nullptr;
	TOptional<FBox> Bounds;
	TArray<uint64> Tags;
	TArray<uint64> DependencyPackageIds;
};

struct UE_EXPERIMENTAL(5.8, "World Streaming Insights API is experimental and subject to change.") FStreamingSourceInfo
{
	uint64 SourceId = 0;
	const TCHAR* Name = nullptr;
	double StartTime = 0.0;
	double EndTime = 0.0;
};

struct UE_EXPERIMENTAL(5.8, "World Streaming Insights API is experimental and subject to change.") FStreamingTagGroup
{
	uint64 GroupId = 0;
	const TCHAR* Name = nullptr;
	const TCHAR* UntaggedLabel = nullptr;
};

struct UE_EXPERIMENTAL(5.8, "World Streaming Insights API is experimental and subject to change.") FStreamingTag
{
	uint64 TagId = 0;
	uint64 GroupId = 0;
	uint64 ParentId = 0;
	const TCHAR* Name = nullptr;
};

struct UE_EXPERIMENTAL(5.8, "World Streaming Insights API is experimental and subject to change.") FStreamingContainerStateChange
{
	double Timestamp = 0.0;
	uint64 ContainerId = 0;
	EStreamingContainerState NewState = EStreamingContainerState::Unloaded;
};

struct UE_EXPERIMENTAL(5.8, "World Streaming Insights API is experimental and subject to change.") FStreamingContainerPriorityUpdate
{
	double Timestamp = 0.0;
	float Priority = 0.0f;
};

struct UE_EXPERIMENTAL(5.8, "World Streaming Insights API is experimental and subject to change.") FStreamingSourceUpdate
{
	double Timestamp = 0.0;
	uint64 SourceId = 0;
	FVector Location = FVector::ZeroVector;
};

class UE_EXPERIMENTAL(5.8, "World Streaming Insights API is experimental and subject to change.") IWorldStreamingInsightsProvider : public TraceServices::IProvider
{
public:
	virtual uint32 GetChangeSerial() const =0;

	virtual uint32 GetStreamingWorldCount() const =0;
	virtual void EnumerateStreamingWorlds(TFunctionRef<void(const FStreamingWorldInfo&)> InCallback) const =0;
	virtual void EnumerateStreamingWorldsAtTime(double InTime, TFunctionRef<void(const FStreamingWorldInfo&)> InCallback) const =0;
	virtual bool ReadStreamingWorld(uint64 InWorldId, TFunctionRef<void(const FStreamingWorldInfo&)> InCallback) const =0;

	virtual uint32 GetStreamingContainerCount(uint64 InWorldId) const =0;
	virtual void EnumerateStreamingContainers(uint64 InWorldId, TFunctionRef<void(const FStreamingContainerInfo&)> InCallback) const =0;
	virtual bool ReadStreamingContainer(uint64 InWorldId, uint64 InContainerId, TFunctionRef<void(const FStreamingContainerInfo&)> InCallback) const =0;
	virtual void EnumerateChildStreamingContainers(uint64 InWorldId, uint64 InContainerId, TFunctionRef<void(const FStreamingContainerInfo&)> InCallback) const =0;
	virtual void EnumerateRootStreamingContainers(uint64 InWorldId, TFunctionRef<void(const FStreamingContainerInfo&)> InCallback) const =0;

	virtual void EnumerateStreamingContainerStateChanges(uint64 InWorldId, uint64 InContainerId, double InStartTime, double InEndTime, TFunctionRef<void(const FStreamingContainerStateChange&)> InCallback) const =0;
	virtual EStreamingContainerState GetStreamingContainerStateAtTime(uint64 InWorldId, uint64 InContainerId, double InTime) const =0;

	virtual float GetStreamingContainerPriorityAtTime(uint64 InWorldId, uint64 InContainerId, double InTime) const =0;

	virtual void EnumerateStreamingSources(uint64 InWorldId, TFunctionRef<void(const FStreamingSourceInfo&)> InCallback) const =0;
	virtual void EnumerateStreamingSourcesAtTime(uint64 InWorldId, double InTime, TFunctionRef<void(const FStreamingSourceInfo&)> InCallback) const =0;
	virtual bool ReadStreamingSourceAtTime(uint64 InWorldId, uint64 InSourceId, double InTime, TFunctionRef<void(const FStreamingSourceInfo&)> InCallback) const =0;

	virtual void EnumerateStreamingSourceUpdates(uint64 InWorldId, uint64 InSourceId, double InStartTime, double InEndTime, TFunctionRef<void(const FStreamingSourceUpdate&)> InCallback) const =0;
	virtual FVector GetStreamingSourceLocationAtTime(uint64 InWorldId, uint64 InSourceId, double InTime) const =0;

	virtual void EnumerateStreamingTagGroups(TFunctionRef<void(const FStreamingTagGroup&)> InCallback) const =0;
	virtual bool ReadStreamingTagGroup(uint64 InGroupId, TFunctionRef<void(const FStreamingTagGroup&)> InCallback) const =0;

	virtual void EnumerateStreamingTagsInGroup(uint64 InGroupId, TFunctionRef<void(const FStreamingTag&)> InCallback) const =0;
	virtual bool ReadStreamingTag(uint64 InTagId, TFunctionRef<void(const FStreamingTag&)> InCallback) const =0;

	virtual const TCHAR* GetPackageName(uint64 InPackageId) const =0;
};

UE_EXPERIMENTAL(5.8, "World Streaming Insights API is experimental and subject to change.")
UE_API FName GetWorldStreamingInsightsProviderName();
UE_EXPERIMENTAL(5.8, "World Streaming Insights API is experimental and subject to change.")
UE_API const IWorldStreamingInsightsProvider* ReadWorldStreamingInsightsProvider(const TraceServices::IAnalysisSession& InSession);

#undef UE_API