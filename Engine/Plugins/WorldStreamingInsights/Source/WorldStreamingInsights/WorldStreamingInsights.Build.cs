// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class WorldStreamingInsights : ModuleRules
	{
		public WorldStreamingInsights(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"ApplicationCore",
					"Core",
					"CoreUObject",
					"InputCore",
					"Slate",
					"SlateCore",
					"TraceAnalysis",
					"TraceInsights",
					"TraceInsightsCore",
					"TraceServices"
				}
			);
		}
	}
}